{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PCGToolset",
	"Description": "A collection of tools allowing the assistant to create and modify PCG graphs",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCGToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}