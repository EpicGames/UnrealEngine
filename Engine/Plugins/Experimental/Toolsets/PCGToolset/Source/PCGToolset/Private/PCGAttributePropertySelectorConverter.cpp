// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGAttributePropertySelectorConverter.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Metadata/PCGAttributePropertySelector.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/UnrealType.h"

namespace
{
	const FPCGAttributePropertySelector* AsSelector(const void* Value)
	{
		return static_cast<const FPCGAttributePropertySelector*>(Value);
	}

	FPCGAttributePropertySelector* AsSelector(void* Value)
	{
		return static_cast<FPCGAttributePropertySelector*>(Value);
	}

	bool IsSelectorStruct(const UStruct* Struct)
	{
		for (const UStruct* Current = Struct; Current; Current = Current->GetSuperStruct())
		{
			if (Current == FPCGAttributePropertySelector::StaticStruct())
			{
				return true;
			}
		}

		return false;
	}
}

namespace UE::PCGToolset
{
	FString FPCGAttributePropertySelectorConverter::GetName() const
	{
		static FString Name(TEXT("PCGAttributePropertySelectorConverter"));
		return Name;
	}

	bool FPCGAttributePropertySelectorConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			return IsSelectorStruct(StructProperty->Struct);
		}

		return false;
	}

	TSharedPtr<FJsonObject> FPCGAttributePropertySelectorConverter::PropertyToJsonSchema(TNotNull<const FProperty*> Property)
	{
		TSharedRef<FJsonObject> Out = MakeShared<FJsonObject>();
		Out->SetStringField(TEXT("type"), TEXT("string"));
		Out->SetStringField(TEXT("title"), TEXT("PCGAttributePropertySelector"));
		Out->SetStringField(
			TEXT("description"),
			TEXT("PCG attribute/property selector in DSL form. ")
			TEXT("Examples: '$Density', 'MyAttr', '@Elements.MyAttr', 'MyAttr.X.Y'"));
		return Out;
	}

	TSharedPtr<FJsonValue> FPCGAttributePropertySelectorConverter::PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (!StructProperty)
		{
			return MakeShared<FJsonValueString>(DefaultString);
		}

		// Construct a temporary selector on the property's struct (to preserve Input/Output
		// default state if needed) and round-trip the incoming default through its own
		// import path, so we emit a canonical DSL string regardless of whether the
		// DefaultString is raw DSL or wrapped in PCG's ExportText sentinels.
		TArray<uint8> Storage;
		Storage.SetNumZeroed(StructProperty->Struct->GetStructureSize());
		StructProperty->Struct->InitializeStruct(Storage.GetData());

		FPCGAttributePropertySelector* Selector = AsSelector(Storage.GetData());

		if (!DefaultString.IsEmpty())
		{
			// Try ImportText first (handles "PCGBegin(...)PCGEnd" form); if it fails,
			// treat the string as raw DSL.
			FStringOutputDevice SilentErrors;
			const TCHAR* Consumed = Property->ImportText_Direct(
				*DefaultString, Storage.GetData(), nullptr, PPF_None, &SilentErrors);
			if (!Consumed)
			{
				Selector->Update(FStringView(DefaultString));
			}
		}

		FString Canonical = Selector->ToString();

		StructProperty->Struct->DestroyStruct(Storage.GetData());

		return MakeShared<FJsonValueString>(MoveTemp(Canonical));
	}

	TSharedPtr<FJsonValue> FPCGAttributePropertySelectorConverter::PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value)
	{
		const FPCGAttributePropertySelector* Selector = AsSelector(Value);
		return MakeShared<FJsonValueString>(Selector->ToString());
	}

	bool FPCGAttributePropertySelectorConverter::JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer)
	{
		FString Str;
		if (!JsonValue.IsValid() || !JsonValue->TryGetString(Str))
		{
			return false;
		}

		FPCGAttributePropertySelector* Selector = AsSelector(OutValue);
		Selector->Update(FStringView(Str));
		return true;
	}
}
