// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

namespace UE::PCGToolset
{
	/** Converts FPCGAttributePropertySelector (and its Input/Output derivatives) to and from
	*   the plain PCG DSL string (e.g. "$Density", "@Metadata.MyAttr", "MyAttr.X.Y") so the
	*   LLM sees a single string instead of PCG's internal 6-field selector struct.
	*/
	class FPCGAttributePropertySelectorConverter : public UE::ToolsetRegistry::FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override;
		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;
		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property) override;
		virtual TSharedPtr<FJsonValue> PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString) override;
		virtual TSharedPtr<FJsonValue> PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value) override;
		virtual bool JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer) override;
	};
}
