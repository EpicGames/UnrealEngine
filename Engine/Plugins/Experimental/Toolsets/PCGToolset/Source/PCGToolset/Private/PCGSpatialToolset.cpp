// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGSpatialToolset.h"

#include "PCGToolsetLibraryCore.h"

#include "PCGDefaultActorExecutionSource.h"
#include "PCGGraph.h"
#include "PCGSubsystem.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "StructUtils/PropertyBag.h"

UPCGExecuteGraphInstanceAsyncResult* UPCGSpatialToolset::RunPCGInstantGraph(UPCGGraph* Graph, const TMap<FString, FString>& Params)
{
	UPCGExecuteGraphInstanceAsyncResult* AsyncResult = NewObject<UPCGExecuteGraphInstanceAsyncResult>();

	if (!ensure(IsInGameThread()))
	{
		AsyncResult->SetError(TEXT("Error: This needs to be run from the game thread."));
		return AsyncResult;
	}

	if (!Graph)
	{
		AsyncResult->SetError(TEXT("Error: Graph is null"));
		return AsyncResult;
	}

	UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld();
	if (!PCGSubsystem)
	{
		AsyncResult->SetError(TEXT("Error: Failed to get PCGSubsystem for current world"));
		return AsyncResult;
	}

	// Create a transient graph instance to avoid modifying the source asset.
	// Do NOT use UPCGGraphInstance::CreateInstance here. CreateInstance hardcodes RF_Transactional |
	// RF_Public, then calls SetGraph which calls Modify(). With RF_Transactional set, Modify() saves
	// the object into the undo transaction buffer as a strong TObjectPtr reference. Each call
	// accumulates such entries; when the undo buffer eventually flushes they all release simultaneously,
	// flooding the GC with many objects at once and corrupting FUObjectHashTables before
	// ShrinkUObjectHashTables runs. Creating with RF_Transient instead makes Modify() a no-op
	// (UObject::Modify checks RF_Transactional before recording) and prevents the accumulation.
	UPCGGraphInstance* GraphInstance = NewObject<UPCGGraphInstance>(PCGSubsystem, MakeUniqueObjectName(PCGSubsystem, UPCGGraphInstance::StaticClass(), Graph->GetFName()), RF_Transient);
	if (!GraphInstance)
	{
		AsyncResult->SetError(TEXT("Error: Failed to create graph instance"));
		return AsyncResult;
	}

	GraphInstance->SetGraph(Graph);

	const FString GraphName = Graph->GetFName().ToString();
	const FInstancedPropertyBag* PropertyBag = GraphInstance->GetUserParametersStruct();
	if (!PropertyBag)
	{
		AsyncResult->SetError(FString::Printf(TEXT("Error: Failed to get property bag for graph '%s'"), *GraphName));
		return AsyncResult;
	}

	TArray<FPCGNodeExecutionMessage> WarningMessages;
	TSharedPtr<FJsonObject> ValidParams = MakeShared<FJsonObject>();
	for (const TPair<FString, FString>& Param : Params)
	{
		const FName ParamName(*Param.Key);
		if (!PropertyBag->FindPropertyDescByName(ParamName))
		{
			FPCGNodeExecutionMessage& Warning = WarningMessages.AddDefaulted_GetRef();
			Warning.Message = FString::Printf(TEXT("Warning: Parameter '%s' not found on graph '%s'"), *Param.Key, *GraphName);
			Warning.Severity = PCGToolsetLibrary::Graph::VerbosityToString(ELogVerbosity::Warning);
			continue;
		}

		TSharedPtr<FJsonValue> JsonValue;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Param.Value);
		if (!FJsonSerializer::Deserialize(Reader, JsonValue) || !JsonValue.IsValid())
		{
			JsonValue = MakeShared<FJsonValueString>(Param.Value);
		}

		ValidParams->SetField(Param.Key, JsonValue);
	}

	if (ValidParams->Values.Num() > 0)
	{
		PCGToolsetLibrary::Graph::SetGraphInstanceParams(GraphInstance, PCGToolsetLibrary::Json::ToJsonString(ValidParams));
	}

	FPCGDefaultWorldObjectExecutionSourceParams ExecutionParams;
	ExecutionParams.GraphInterface = GraphInstance;
	ExecutionParams.WorldObject = PCGSubsystem;
	ExecutionParams.bFireAndForgetExecution = true;
	ExecutionParams.GenerationCallback = FPCGOnEditorGenerationDone::CreateLambda(
		[WarningMessages, AsyncResult = TStrongObjectPtr(AsyncResult)](
			IPCGGraphExecutionSource* ExecutionSource, EPCGGenerationStatus Status)
		{
			ensure(IsInGameThread());
			TArray<FPCGNodeExecutionMessage> Messages = WarningMessages;

			const PCGUtils::FExtraCapture& Capture = ExecutionSource->GetExecutionState().GetExtraCapture();
			const TMap<TWeakObjectPtr<const UPCGNode>, TArray<PCGUtils::FCapturedMessage>>& NodeToMessageMap = Capture.GetCapturedMessages();
			for (const TPair<TWeakObjectPtr<const UPCGNode>, TArray<PCGUtils::FCapturedMessage>>& Pair : NodeToMessageMap)
			{
				const UPCGNode* Node = Pair.Key.Get();
				if (!Node)
				{
					continue;
				}
				const FString NodeName = Node->GetFName().ToString();
				for (const PCGUtils::FCapturedMessage& CapturedMessage : Pair.Value)
				{
					FPCGNodeExecutionMessage& Entry = Messages.AddDefaulted_GetRef();
					Entry.Message = CapturedMessage.Message;
					Entry.Severity = PCGToolsetLibrary::Graph::VerbosityToString(CapturedMessage.Verbosity);
					Entry.ReporterNode = NodeName;
				}
			}

			if (Status == EPCGGenerationStatus::Completed)
			{
				AsyncResult->SetValue(MoveTemp(Messages));
			}
			else if (Messages.IsEmpty())
			{
				AsyncResult->SetError(TEXT("Error: Graph execution aborted. Unknown Issue"));
			}
			else
			{
				TArray<TSharedPtr<FJsonValue>> MessageJsonValues;
				MessageJsonValues.Reserve(Messages.Num());
				for (const FPCGNodeExecutionMessage& Msg : Messages)
				{
					TSharedPtr<FJsonObject> MessageJson = MakeShared<FJsonObject>();
					MessageJson->SetStringField(TEXT("message"), Msg.Message);
					MessageJson->SetStringField(TEXT("severity"), Msg.Severity);
					MessageJson->SetStringField(TEXT("reporter_node"), Msg.ReporterNode);
					MessageJsonValues.Add(MakeShared<FJsonValueObject>(MoveTemp(MessageJson)));
				}
				AsyncResult->SetError(FString::Printf(TEXT("Error: Graph execution aborted. ErrorMessages: '%s'"),
					*PCGToolsetLibrary::Json::ToJsonString(MessageJsonValues)));
			}
		}
	);

	UPCGDefaultWorldObjectExecutionSource* ExecutionSource = IPCGBaseSubsystem::CreateExecutionSource<UPCGDefaultWorldObjectExecutionSource>(ExecutionParams);
	if (!ExecutionSource)
	{
		AsyncResult->SetError(TEXT("Error: Failed to create execution source"));
	}

	return AsyncResult;
}
