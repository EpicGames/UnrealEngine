// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGToolset.h"

#include "PCGToolsetLibraryCore.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGSubgraph.h"
#include "PCGVolume.h"
#include "Data/DataView/PCGDataView.h"
#include "Data/DataView/PCGDataViewData.h"
#include "Data/DataView/PCGDataViewNativeJsonConverters.h"
#include "Elements/PCGFilterByType.h"
#include "Helpers/PCGHelpers.h"
#include "Helpers/PCGSubgraphHelpers.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "AssetToolsModule.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "IAssetTools.h"
#include "ScopedTransaction.h"
#include "ActorFactories/ActorFactory.h"
#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Builders/CubeBuilder.h"
#include "Components/BrushComponent.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/PackageName.h"
#include "ToolsetRegistry/ToolsetLibrary.h"

namespace PCGToolset
{
	FIntRect GetNodesBoundBox(const TArray<UPCGNode*>& Nodes)
	{
		const FIntPoint NodeSafeSize(200, 100);
		TArray<FIntRect> NodeBoxes;
		Algo::Transform(Nodes, NodeBoxes, [&NodeSafeSize](const UPCGNode* Node) -> FIntRect
		{
			FIntPoint NodePos{Node->PositionX, Node->PositionY};
			return FIntRect(NodePos, NodePos + NodeSafeSize);
		});

		FIntRect CommentBox = NodeBoxes[0];
		for (const FIntRect& NodeBox : NodeBoxes)
		{
			CommentBox.Union(NodeBox);
		}

		CommentBox.InflateRect(50);

		return CommentBox;
	}

	bool SafeRenameNode(UPCGNode* Node, const FString& NewName)
	{
		// Only do the real call if the test succeeds.
		return Node->Rename(*NewName, nullptr, REN_Test) && Node->Rename(*NewName);
	}

	TOptional<TArray<UPCGNode*>> TryAddConversionNodes(UPCGGraph* Graph, UPCGPin* OutPin, UPCGPin* InPin)
	{
		check(Graph && OutPin && InPin);
		const FPCGDataTypeRegistry& Registry = FPCGModule::GetConstDataTypeRegistry();
		const FPCGDataTypeIdentifier UpstreamTypes = OutPin->GetCurrentTypesID();
		const FPCGDataTypeIdentifier DownstreamTypes = InPin->GetCurrentTypesID();

		for (const FPCGDataTypeBaseId& Id : UpstreamTypes.GetIds())
		{
			if (const FPCGDataTypeInfo* Info = Registry.GetTypeInfo(Id))
			{
				if (TOptional<TArray<UPCGNode*>> Nodes = Info->AddConversionNodesTo(UpstreamTypes, DownstreamTypes, Graph, OutPin, InPin))
				{
					return Nodes;
				}
			}
		}

		for (const FPCGDataTypeBaseId& Id : DownstreamTypes.GetIds())
		{
			if (const FPCGDataTypeInfo* Info = Registry.GetTypeInfo(Id))
			{
				if (TOptional<TArray<UPCGNode*>> Nodes = Info->AddConversionNodesFrom(UpstreamTypes, DownstreamTypes, Graph, OutPin, InPin))
				{
					return Nodes;
				}
			}
		}

		return {};
	}
}

// Graph -----------------------------------------------------------------------------------------------------------
UPCGGraph* UPCGToolset::CreateGraph(const FString& Name, const FString& Path)
{
	if (Name.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError("Name parameter must be provided");
		return nullptr;
	}

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	FString GraphUniqueAssetName;
	FString GraphPackageNameToUse;
	const FString FullPath = Path / Name;
	AssetToolsModule.Get().CreateUniqueAssetName(FullPath, FString(), GraphPackageNameToUse, GraphUniqueAssetName);
	const FString PackagePath = FPackageName::GetLongPackagePath(GraphPackageNameToUse);

	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(GraphUniqueAssetName, PackagePath, UPCGGraph::StaticClass(), nullptr);
	UPCGGraph* Graph = Cast<UPCGGraph>(NewAsset);

	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Could not create PCG Graph");
		return nullptr;
	}

	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	return Graph;
}

// Graph Params -----------------------------------------------------------------------------------------------------
FPCGGraphStructure UPCGToolset::GetGraphStructure(const UPCGGraph* Graph)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return {};
	}

	FPCGGraphStructure Result;
	Result.Name = Graph->GetFName().ToString();
	Result.Description = Graph->Description.ToString();
	Result.Nodes = Graph::GetGraphNodesInfo(Graph);
	Result.Edges = Graph::GetGraphEdges(Graph);
	return Result;
}

bool UPCGToolset::SetGraphParams(UPCGGraph* Graph, const TArray<FPCGParamDefinition>& Params)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	TArray<FPropertyBagPropertyDesc> NewDescs;
	FString DefaultValueFields;
	for (const FPCGParamDefinition& Param : Params)
	{
		FPropertyBagPropertyDesc NewDesc = PCGPropertyHelpers::CreatePropertyBagDescWithMetadataType(Param.Name, Param.Type);
		NewDesc.SetMetaData(TEXT("ToolTip"), Param.Description);
		NewDesc.ContainerTypes = FPropertyBagContainerTypes{Param.ContainerType};
		NewDescs.Add(NewDesc);

		// Native property-bag initialization provides correct defaults (zero/identity) for every
		// supported type, so we only need to override when the caller supplied a custom value.
		if (Param.DefaultValueJson.IsEmpty())
		{
			continue;
		}

		if (!DefaultValueFields.IsEmpty())
		{
			DefaultValueFields += TEXT(",");
		}

		DefaultValueFields += FString::Printf(TEXT("\"%s\":%s"), *Param.Name.ToString(), *Param.DefaultValueJson);
	}

	EPropertyBagAlterationResult Result = Graph->AddUserParameters(NewDescs);
	if (Result != EPropertyBagAlterationResult::Success)
	{
		FString ResultValue = StaticEnum<EPropertyBagAlterationResult>()->GetNameStringByValue(static_cast<int64>(Result));
		FString ErrorMessage = FString::Format(TEXT("Error: Adding parameters failed due to -> {0}"), {ResultValue});
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return false;
	}

	bool bSuccess = true;
	if (!DefaultValueFields.IsEmpty())
	{
		// Wrap as {"UserParameters": {...}} so SetObjectProperties descends into the graph's bag.
		const FString FullJson = FString::Printf(TEXT("{\"UserParameters\":{%s}}"), *DefaultValueFields);
		bSuccess = UToolsetLibrary::SetObjectProperties(Graph, FullJson, EBypassContainerCheck::Yes);
	}

	Graph->OnGraphParametersChanged(EPCGGraphParameterEvent::MultiplePropertiesAdded, NAME_None);

	return bSuccess;
}

bool UPCGToolset::RemoveGraphParams(UPCGGraph* Graph, const TArray<FName>& ParamNames)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	FInstancedPropertyBag* PropertyBag = Graph->GetMutableUserParametersStruct_Unsafe();
	if (!PropertyBag)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph has no user parameters bag");
		return false;
	}

	EPropertyBagAlterationResult Result = PropertyBag->RemovePropertiesByName(ParamNames);
	if (Result != EPropertyBagAlterationResult::Success)
	{
		FString ResultValue = StaticEnum<EPropertyBagAlterationResult>()->GetNameStringByValue(static_cast<int64>(Result));
		FString ErrorMessage = FString::Format(TEXT("Error: Removing parameters failed due to -> {0}"), {ResultValue});
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return false;
	}

	Graph->OnGraphParametersChanged(EPCGGraphParameterEvent::RemovedUsed, NAME_None);
	return true;
}

FPCGGraphSchema UPCGToolset::GetGraphSchema(const UPCGGraph* Graph)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return {};
	}

	FPCGGraphSchema Result;
	Result.Name = Graph->GetFName().ToString();

	FInstancedPropertyBag FilteredBag = Graph::GetGraphParams(Graph);
	if (const UPropertyBag* BagStruct = FilteredBag.GetPropertyBagStruct())
	{
		Result.GraphParamsSchema = UToolsetLibrary::ListStructProperties(const_cast<UPropertyBag*>(BagStruct), /*bUserVisiblePropertiesOnly=*/false);
	}

	Result.InputPins = Graph::GetNodePinsSchema(Graph->GetInputNode()->OutputPinProperties());
	Result.OutputPins = Graph::GetNodePinsSchema(Graph->GetOutputNode()->InputPinProperties());
	return Result;
}

FString UPCGToolset::GetGraphDescription(const UPCGGraph* Graph)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return {};
	}
	return Graph->Description.ToString();
}

bool UPCGToolset::SetGraphDescription(UPCGGraph* Graph, const FString& Description)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("SetGraphDescriptionToolCall"));
	Graph->Modify();
	Graph->Description = FText::FromString(Description);
	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	return true;
}

// Graph Instance --------------------------------------------------------------------------------------------------
TArray<FPCGGraphInstanceInfo> UPCGToolset::ListGraphInstances()
{
	TArray<FPCGGraphInstanceInfo> Result;
	const UWorld* World = GEditor ? GEditor->GetEditorWorldContext(true).World() : nullptr;
	if (!World)
	{
		return Result;
	}

	for (TActorIterator<APCGVolume> It(World); It; ++It)
	{
		APCGVolume* PCGVolume = *It;
		if (!PCGVolume || !PCGVolume->PCGComponent)
		{
			continue;
		}

		if (UPCGGraphInstance* GraphInstance = PCGVolume->PCGComponent->GetGraphInstance())
		{
			FPCGGraphInstanceInfo Info;
			Info.Actor = PCGVolume;
			Info.Graph = PCGVolume->PCGComponent->GetGraph();
			Result.Add(Info);
		}
	}
	return Result;
}

FPCGGraphInstanceInfo UPCGToolset::SpawnGraphInstance(UPCGGraph* Graph, const FString& Name, const FTransform& Transform, const FString& JsonParams)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return {};
	}

	UWorld* World = GEditor ? GEditor->GetEditorWorldContext(true).World() : nullptr;
	if (!World)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: EditorWorldContext not found");
		return {};
	}

	FActorSpawnParameters ActorSpawnParams;
	const FName ActorName(Name);
	ActorSpawnParams.Name = MakeUniqueObjectName(World, APCGVolume::StaticClass(), ActorName, EUniqueObjectNameOptions::UniversallyUnique);
	ActorSpawnParams.InitialActorLabel = Name;

	APCGVolume* PCGVolume = World->SpawnActor<APCGVolume>(Transform.GetLocation(), Transform.Rotator(), ActorSpawnParams);
	if (!PCGVolume)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Spawning PCGVolume in the world failed");
		return {};
	}

	PCGVolume->SetActorScale3D(Transform.GetScale3D());
	PCGVolume->SetActorLabel(Name);

	// Using ActorFactory/CubeBuilder to create valid BrushComp with valid bounds
	// editor-equivalent to drag & drop.
	// Makes it possible to render with custom depth stencil.
	UCubeBuilder* CubeBuilder = NewObject<UCubeBuilder>(GetTransientPackage());
	CubeBuilder->X = 200.f;
	CubeBuilder->Y = 200.f;
	CubeBuilder->Z = 200.f;
	CubeBuilder->Hollow = false;

	UActorFactory::CreateBrushForVolumeActor(PCGVolume, CubeBuilder);
	if (UBrushComponent* BrushComp = PCGVolume->GetBrushComponent())
	{
		BrushComp->ReregisterComponent();
		BrushComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		BrushComp->SetCanEverAffectNavigation(false);
	}

	// Set the PCG Component on the graph
	UPCGComponent* Component = PCGVolume->PCGComponent;
	if (!Component)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Could not find PCG Component on PCGVolume");
		PCGVolume->Destroy();
		return {};
	}

	Component->SetGraph(Graph);

	// Finally, set it's graph params
	if (!JsonParams.IsEmpty())
	{
		if (!Graph::SetGraphInstanceParams(Component->GetGraphInstance(), JsonParams))
		{
			UKismetSystemLibrary::RaiseScriptError("Warning: Graph instance was spawned, but there was an error when setting graph params");
		}
	}

	FPCGGraphInstanceInfo GraphInstanceInfo;
	GraphInstanceInfo.Actor = PCGVolume;
	GraphInstanceInfo.Graph = Graph;
	return GraphInstanceInfo;
}

// Graph Instance Param --------------------------------------------------------------------------------------------
FInstancedPropertyBag UPCGToolset::GetGraphInstanceParams(const APCGVolume* PCGVolume)
{
	using namespace PCGToolsetLibrary;
	if (!PCGVolume || !PCGVolume->PCGComponent || !PCGVolume->PCGComponent->GetGraphInstance())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: PCGVolume is invalid"));
		return {};
	}

	const FInstancedPropertyBag& PropertyBag = PCGVolume->PCGComponent->GetGraphInstance()->ParametersOverrides.Parameters;
	if (!PropertyBag.IsValid())
	{
		return {};
	}

	return Graph::BuildFilteredBag(PropertyBag);
}

bool UPCGToolset::SetGraphInstanceParams(const APCGVolume* PCGVolume, const FString& JsonParams)
{
	using namespace PCGToolsetLibrary;
	if (!PCGVolume)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: PCGVolume not found");
		return false;
	}

	if (!PCGVolume->PCGComponent)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: PCGVolume has no PCG component");
		return false;
	}

	UPCGGraphInstance* InOutGraphInstance = PCGVolume->PCGComponent->GetGraphInstance();
	if (!InOutGraphInstance)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: The graph instance on the PCG Volume is invalid");
		return false;
	}

	return Graph::SetGraphInstanceParams(InOutGraphInstance, JsonParams);
}

bool UPCGToolset::ResetGraphInstanceParams(const APCGVolume* PCGVolume, const TArray<FName>& ParamNames)
{
	if (!PCGVolume || !PCGVolume->PCGComponent || !PCGVolume->PCGComponent->GetGraphInstance())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: PCGVolume is invalid"));
		return false;
	}

	UPCGGraphInstance* GraphInstance = PCGVolume->PCGComponent->GetGraphInstance();
	FInstancedPropertyBag& Parameters = GraphInstance->ParametersOverrides.Parameters;

	TArray<FString> NotFound;
	for (const FName& ParamName : ParamNames)
	{
		if (const FPropertyBagPropertyDesc* Param = Parameters.FindPropertyDescByName(ParamName))
		{
			GraphInstance->UpdatePropertyOverride(Param->CachedProperty, false);
		}
		else
		{
			NotFound.Add(ParamName.ToString());
		}
	}

	if (!NotFound.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("ResetGraphInstanceParams: the following parameters were not found: %s"), *FString::Join(NotFound, TEXT(", "))));
		return false;
	}
	return true;
}

// Node Search -----------------------------------------------------------------------------------------------------
TArray<FString> UPCGToolset::ListNativeNodes(const bool bCommonOnly)
{
	using namespace PCGToolsetLibrary;
	TArray<FString> Nodes;

	const TMap<FName, UPCGSettings*>& NodeNameToSettingsMap = Graph::GetNodeNameToSettingsMap();
	for (const auto& [Name, Settings] : NodeNameToSettingsMap)
	{
		if (bCommonOnly && !Constants::CommonNativeNodes.Contains(Name.ToString()))
		{
			continue;
		}

		Nodes.Add(Name.ToString());
	}

	return Nodes;
}

TArray<FString> UPCGToolset::ListAvailableSubgraphs()
{
	using namespace PCGToolsetLibrary;
	return Graph::FindGraphPaths(Constants::GetSubgraphDirectories(), [](const FString& PathName)
	{
		return !PathName.Contains(TEXT("Subgraphs/"))
			&& !PathName.Contains(TEXT("Shared/"))
			&& !PathName.Contains(TEXT("_Template_"));
	});
}

// Nodes ----------------------------------------------------------------------------------------------------------
FPCGNativeNodeSchema UPCGToolset::GetNativeNodeSchema(const FString& NodeName)
{
	using namespace PCGToolsetLibrary;
	if (NodeName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError("Error: NodeName is empty");
		return {};
	}

	const TMap<FName, UPCGSettings*>& NodeNameToSettingsMap = Graph::GetNodeNameToSettingsMap();

	UPCGSettings* const* SettingsPtr = NodeNameToSettingsMap.Find(FName(NodeName));
	if (SettingsPtr == nullptr || *SettingsPtr == nullptr)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Error: NodeName %s is not in available nodes"), *NodeName));
		return {};
	}

	UPCGSettings* Settings = *SettingsPtr;
	FPCGNativeNodeSchema Result;
	Result.Name = NodeName;
	Result.Description = Settings->GetNodeTooltipText().ToString();

	TArray<FProperty*> NodeProperties = Graph::GetNodePropertiesFromSettings(Settings->GetClass());
	TArray<FPropertyBagPropertyDesc> Descs;
	for (FProperty* Property : NodeProperties)
	{
		Descs.Emplace(Property->GetFName(), Property);
	}

	FInstancedPropertyBag SchemaBag;
	SchemaBag.AddProperties(Descs);
	if (const UPropertyBag* BagStruct = SchemaBag.GetPropertyBagStruct())
	{
		Result.PropertiesSchema = UToolsetLibrary::ListStructProperties(const_cast<UPropertyBag*>(BagStruct), /*bUserVisiblePropertiesOnly=*/false);
	}

	Result.InputPins = Graph::GetNodePinsSchema(Settings->InputPinProperties());
	Result.OutputPins = Graph::GetNodePinsSchema(Settings->OutputPinProperties());
	return Result;
}

UPCGNode* UPCGToolset::AddNode(
	UPCGGraph* Graph,
	const FString& NativeNodeType,
	const FString& NodeName,
	const FString& JsonParams,
	const FString& NodeTitle,
	const FString& NodeComment,
	const int32 XPositionIdx,
	const int32 YPositionIdx)
{
	PCGUtils::FScopedCallOutputDevice OutputDevice;
	PCGUtils::FScopedCall ScopedCall(nullptr, nullptr, OutputDevice);

	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return nullptr;
	}

	const TMap<FName, UPCGSettings*>& AvailableNodes = Graph::GetNodeNameToSettingsMap();
	UPCGSettings* const* FoundSettings = AvailableNodes.Find(FName(NativeNodeType));

	if (!FoundSettings)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Error: Node type '{0}' does not exist"), {NativeNodeType}));
		return nullptr;
	}

	FScopedTransaction Transaction(INVTEXT("AddGraphNodeToolCall"));
	Graph->Modify();

	UPCGSettings* DefaultSettings = nullptr;
	UPCGNode* CreatedNode = Graph->AddNodeOfType((*FoundSettings)->GetClass(), DefaultSettings);
	if (!CreatedNode)
	{
		Transaction.Cancel();
		Graph::RaiseScopedErrors(ScopedCall);
		FString ErrorMessage = FString::Format(TEXT("Unable to create node with type '{0}'."), {NativeNodeType});
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return nullptr;
	}

	auto OnErrorCleanup = [&]()
	{
		Graph::RaiseScopedErrors(ScopedCall);
		Transaction.Cancel();
		Graph->RemoveNode(CreatedNode);
	};

	CreatedNode->NodeTitle = FName(NodeTitle);
	CreatedNode->NodeComment = NodeComment;
	CreatedNode->bCommentBubblePinned = false;
	CreatedNode->bCommentBubbleVisible = !NodeComment.IsEmpty();

	if (!PCGToolset::SafeRenameNode(CreatedNode, NodeName))
	{
		OnErrorCleanup();
		UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Could not name node to '{0}' (name may already be in use)"), {NodeName}));
		return nullptr;
	}

	if (!JsonParams.IsEmpty())
	{
		if (!UToolsetLibrary::SetObjectProperties(CreatedNode->GetSettings(), JsonParams, EBypassContainerCheck::Yes))
		{
			OnErrorCleanup();
			UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to set graph params. Node not added."));
			return nullptr;
		}
	}

	CreatedNode->SetNodePosition(XPositionIdx, YPositionIdx);

	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);

	return CreatedNode;
}

UPCGNode* UPCGToolset::AddSubgraphNode(
	UPCGGraph* Graph,
	UPCGGraph* SubGraphForNode,
	const FString& NodeName,
	const FString& JsonParams,
	const FString& NodeTitle,
	const FString& NodeComment,
	const int32 XPositionIdx,
	const int32 YPositionIdx)
{
	using namespace PCGToolsetLibrary;

	PCGUtils::FScopedCallOutputDevice OutputDevice;
	PCGUtils::FScopedCall ScopedCall(nullptr, nullptr, OutputDevice);

	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return nullptr;
	}

	if (!SubGraphForNode)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No PCGGraph asset for subgraph");
		return nullptr;
	}

	FScopedTransaction Transaction(INVTEXT("AddGraphNodeToolCall"));
	Graph->Modify();

	UPCGSubgraphSettings* SubgraphSettings = nullptr;
	UPCGNode* CreatedNode = Graph->AddNodeOfType(SubgraphSettings);
	if (!CreatedNode)
	{
		Graph::RaiseScopedErrors(ScopedCall);
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(TEXT("Unable to create subgraph node."));
		return nullptr;
	}

	auto OnErrorCleanup = [&]()
	{
		Graph::RaiseScopedErrors(ScopedCall);
		Transaction.Cancel();
		Graph->RemoveNode(CreatedNode);
	};

	CreatedNode->NodeTitle = FName(NodeTitle);
	CreatedNode->NodeComment = NodeComment;
	CreatedNode->bCommentBubblePinned = false;
	CreatedNode->bCommentBubbleVisible = !NodeComment.IsEmpty();

	if (!PCGToolset::SafeRenameNode(CreatedNode, NodeName))
	{
		OnErrorCleanup();
		UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Could not name node to '{0}' (name may already be in use)"), {NodeName}));
		return nullptr;
	}

	SubgraphSettings->SetSubgraph(SubGraphForNode);

	UPCGGraphInterface* SubgraphInterface = SubgraphSettings->GetSubgraphInterface();
	if (!ensure(SubgraphInterface && SubgraphInterface->IsInstance()))
	{
		OnErrorCleanup();
		UKismetSystemLibrary::RaiseScriptError(TEXT("Subgraph failed to be instantiated"));
		return nullptr;
	}

	if (!JsonParams.IsEmpty())
	{
		UPCGGraphInstance* SubgraphInstance = Cast<UPCGGraphInstance>(SubgraphInterface);
		if (!Graph::SetGraphInstanceParams(SubgraphInstance, JsonParams))
		{
			OnErrorCleanup();
			UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to set graph params. Node not added."));
			return nullptr;
		}
	}

	CreatedNode->SetNodePosition(XPositionIdx, YPositionIdx);

	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);

	return CreatedNode;
}

bool UPCGToolset::UpdateNode(UPCGNode* Node, const FString& JsonParams, const FString& NodeTitle)
{
	using namespace PCGToolsetLibrary;

	PCGUtils::FScopedCallOutputDevice OutputDevice;
	PCGUtils::FScopedCall ScopedCall(nullptr, nullptr, OutputDevice);

	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node not found");
		return false;
	}

	if (!Node->GetSettings())
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node does not have valid settings");
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("EditGraphNodeToolCall"));
	Node->Modify();

	if (!NodeTitle.IsEmpty())
	{
		Node->NodeTitle = FName(NodeTitle);
	}

	bool bSuccess = true;
	if (!JsonParams.IsEmpty())
	{
		if (UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(Node->GetSettings()))
		{
			UPCGGraphInterface* SubgraphInterface = SubgraphSettings->GetSubgraphInterface();
			if (!ensure(SubgraphInterface && SubgraphInterface->IsInstance()))
			{
				Transaction.Cancel();
				Graph::RaiseScopedErrors(ScopedCall);
				UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Subgraph must be instantiated: {0}"), {Node->GetName()}));
				return false;
			}

			UPCGGraphInstance* SubgraphInstance = Cast<UPCGGraphInstance>(SubgraphInterface);
			SubgraphInstance->Modify();

			// @todo_pcg: params not being set properly should return success: true but with warnings
			if (!Graph::SetGraphInstanceParams(SubgraphInstance, JsonParams))
			{
				Graph::RaiseScopedErrors(ScopedCall);
				Transaction.Cancel();
				UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Could not set params on node '{0}'."), {Node->GetName()}));
				return false;
			}
		}
		else if (!UToolsetLibrary::SetObjectProperties(Node->GetSettings(), JsonParams, EBypassContainerCheck::Yes))
		{
			Graph::RaiseScopedErrors(ScopedCall);
			Transaction.Cancel();
			UKismetSystemLibrary::RaiseScriptError(FString::Format(TEXT("Could not set params on node '{0}'."), {Node->GetName()}));
			return false;
		}
	}

	if (UPCGGraph* Graph = Node->GetGraph())
	{
		[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
		Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	}
	return bSuccess;
}

bool UPCGToolset::SetNodeComment(UPCGNode* Node, const FString& NodeComment)
{
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node not found");
		return false;
	}

	const bool bBubbleVisible = !NodeComment.IsEmpty();

	if (Node->NodeComment == NodeComment && bBubbleVisible == Node->bCommentBubbleVisible)
	{
		// No changes
		return true;
	}

	FScopedTransaction Transaction(INVTEXT("SetNodeComment"));
	Node->Modify();
	Node->NodeComment = NodeComment;
	Node->bCommentBubbleVisible = bBubbleVisible;

	if (UPCGGraph* Graph = Node->GetGraph())
	{
		[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
		Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	}

	return true;
}

FPCGNodeInfo UPCGToolset::GetNodeInfo(const UPCGNode* Node)
{
	using namespace PCGToolsetLibrary;
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node not found");
		return {};
	}

	return Graph::GetNodeInfo(Node);
}

bool UPCGToolset::RepositionNode(UPCGNode* Node, const int32 XPositionIdx, const int32 YPositionIdx)
{
	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node not found");
		return false;
	}

	Node->SetNodePosition(XPositionIdx, YPositionIdx);
	return true;
}

bool UPCGToolset::RemoveNode(UPCGGraph* Graph, UPCGNode* Node)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node not found");
		return false;
	}

	// Check that the node belongs to the graph.
	if (Node->GetGraph() != Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node should belong to the Graph");
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("RemoveGraphNodeToolCall"));
	Graph->RemoveNode(Node);
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	return true;
}

TArray<UPCGNode*> UPCGToolset::ConnectNodePins(UPCGNode* FromNode, const FString& FromPinLabel, UPCGNode* ToNode, const FString& ToPinLabel)
{
	using namespace PCGToolsetLibrary;
	if (!FromNode)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: FromNode not found");
		return {};
	}

	if (!ToNode)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: ToNode not found");
		return {};
	}

	if (ToNode->GetGraph() != FromNode->GetGraph())
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Both nodes should belong to the same graph");
		return {};
	}

	UPCGGraph* Graph = FromNode->GetGraph();
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Target graph is invalid");
		return {};
	}

	// Check pin name validity
	UPCGPin* OutPin = FromNode->GetOutputPin(FName(FromPinLabel));
	if (!OutPin)
	{
		FString ErrorMessage = FString::Format(TEXT("[Add Graph Edge] From node '{0}' does not have an output pin named '{1}' within graph '{2}'"), {FromNode->GetName(), FromPinLabel, Graph->GetName()});
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	UPCGPin* InPin = ToNode->GetInputPin(FName(ToPinLabel));
	if (!InPin)
	{
		FString ErrorMessage = FString::Format(TEXT("[Add Graph Edge] To node '{0}' does not have an input pin named '{1}' within graph '{2}'"), {ToNode->GetName(), ToPinLabel, Graph->GetName()});
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	FScopedTransaction Transaction(INVTEXT("ConnectNodePinsToolCall"));
	Graph->Modify();

	const EPCGDataTypeCompatibilityResult Compatibility = OutPin->GetCompatibilityWithOtherPin(InPin);
	TArray<UPCGNode*> AddedNodes;

	switch (Compatibility)
	{
		case EPCGDataTypeCompatibilityResult::RequireFilter:
		{
			UPCGNode* ConversionPCGNode = FPCGSubgraphHelpers::SpawnNodeAndConnect(Graph, OutPin, InPin, UPCGFilterByTypeSettings::StaticClass(),
				[InPin](UPCGSettings* NodeSettings)
				{
					UPCGFilterByTypeSettings* Settings = CastChecked<UPCGFilterByTypeSettings>(NodeSettings);
					Settings->TargetType = InPin->Properties.AllowedTypes;
					return true;
				});

			if (!ConversionPCGNode)
			{
				Transaction.Cancel();
				UKismetSystemLibrary::RaiseScriptError(FString::Format(
					TEXT("[Add Graph Edge] Output pin '{0}' on node '{1}' is incompatible with input pin '{2}' on node '{3}' within graph '{4}' And filter node failed to be added."),
					{FromPinLabel, FromNode->GetName(), ToPinLabel, ToNode->GetName(), Graph->GetName()}));
				return {};
			}

			AddedNodes.Add(ConversionPCGNode);
			break;
		}

		case EPCGDataTypeCompatibilityResult::RequireConversion:
		{
			TOptional<TArray<UPCGNode*>> Nodes = PCGToolset::TryAddConversionNodes(Graph, OutPin, InPin);
			if (!Nodes)
			{
				Transaction.Cancel();
				UKismetSystemLibrary::RaiseScriptError(FString::Format(
					TEXT("[Add Graph Edge] Output pin '{0}' on node '{1}' is incompatible with input pin '{2}' on node '{3}' within graph '{4}' And conversion node failed to be added."),
					{FromPinLabel, FromNode->GetName(), ToPinLabel, ToNode->GetName(), Graph->GetName()}));
				return {};
			}

			AddedNodes = MoveTemp(*Nodes);
			break;
		}

		case EPCGDataTypeCompatibilityResult::Compatible: // @todo_pcg: we can relax this if needed.
			Graph->AddLabeledEdge(FromNode, FName(FromPinLabel), ToNode, FName(ToPinLabel));
			break;

		default:
			Transaction.Cancel();
			UKismetSystemLibrary::RaiseScriptError(FString::Format(
				TEXT("[Add Graph Edge] Output pin '{0}' on node '{1}' is incompatible with input pin '{2}' on node '{3}' within graph '{4}'."),
				{FromPinLabel, FromNode->GetName(), ToPinLabel, ToNode->GetName(), Graph->GetName()}));
			return {};
	}

	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	return AddedNodes;
}

bool UPCGToolset::DisconnectNodePins(UPCGNode* FromNode, const FString& FromPinLabel, UPCGNode* ToNode, const FString& ToPinLabel)
{
	if (!FromNode)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: FromNode not found");
		return false;
	}

	if (!ToNode)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: ToNode not found");
		return false;
	}

	if (ToNode->GetGraph() != FromNode->GetGraph())
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Both nodes should belong to the same graph");
		return false;
	}

	UPCGGraph* Graph = FromNode->GetGraph();
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Target graph is invalid");
		return false;
	}

	// Check pin name validity
	UPCGPin* OutPin = FromNode->GetOutputPin(FName(FromPinLabel));
	if (!OutPin)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Format(
			TEXT("[Disconnect Node Pins] From node '{0}' does not have an output pin named '{1}' within graph '{2}'"),
			{ FromNode->GetName(), FromPinLabel, Graph->GetName() }));
		return false;
	}

	UPCGPin* InPin = ToNode->GetInputPin(FName(ToPinLabel));
	if (!InPin)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Format(
			TEXT("[Disconnect Node Pins] To node '{0}' does not have an input pin named '{1}' within graph '{2}'"),
			{ ToNode->GetName(), ToPinLabel, Graph->GetName() }));
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("DisconnectNodePinsToolCall"));
	Graph->Modify();
	if (!Graph->RemoveEdge(FromNode, FName(FromPinLabel), ToNode, FName(ToPinLabel)))
	{
		Transaction.Cancel();
		UKismetSystemLibrary::RaiseScriptError(FString::Format(
			TEXT("Unable to remove edge '{0}' on node '{1}' to edge '{2}' on node '{3}' in graph '{4}'"),
			{ *FromPinLabel, *FromNode->GetName(), *ToPinLabel, *ToNode->GetName(), *Graph->GetName() }));
		return false;
	}

	return true;
}

FString UPCGToolset::GetNodeDataView(
	const APCGVolume* PCGVolume,
	const UPCGNode* Node,
	const FString& PinLabel,
	const FString& AttributeName,
	const int32 StartIndex,
	const int32 EndIndex)
{
	using namespace PCGToolsetLibrary;

	if (!PCGVolume)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: PCGVolume not found");
		return {};
	}

	if (!Node)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No Node found at that path");
		return {};
	}

	UPCGComponent* Component = PCGVolume->PCGComponent;
	if (!Component)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No PCG Component found");
		return {};
	}

	const UPCGGraph* Graph = Cast<UPCGGraph>(Node->GetOuter());
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No graph found on the PCG Component");
		return {};
	}

	const bool bIsInputNode = (Node == Graph->GetInputNode());
	const bool bIsOutputNode = (Node == Graph->GetOutputNode());
	const bool bIsInputOutputNode = bIsInputNode || bIsOutputNode;
	bool bFoundInGraph = false;
	if (!bIsInputOutputNode)
	{
		Graph->ForEachNodeRecursively([&](UPCGNode* GraphNode)
		{
			if (GraphNode == Node)
			{
				bFoundInGraph = true;
				return false;
			}

			return true;
		});
	}

	if (!bIsInputOutputNode && !bFoundInGraph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Node does not belong to this graph instance");
		return {};
	}

	if (EndIndex < -1)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Error: EndIndex (%d) is invalid. Use -1 for all elements or a non-negative value."), EndIndex));
		return {};
	}

	// Validate the requested output pin exists on the node
	const UPCGPin* OutputPin = Node->GetOutputPin(FName(PinLabel));
	if (!OutputPin)
	{
		FString AvailablePins = FString::JoinBy(Node->GetOutputPins(), TEXT(", "), [](const UPCGPin* Pin) { return Pin->Properties.Label.ToString(); });

		FString ErrorMessage = FString::Printf(
			TEXT("Error: Node '%s' has no output pin '%s'. Available: %s"),
			*Node->GetFName().ToString(),
			*PinLabel, *AvailablePins);
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	// Enable inspection and get inspection data
	FPCGGraphExecutionInspection& Inspection = Component->GetExecutionState().GetInspection();
	if (!Inspection.IsInspecting())
	{
		Inspection.EnableInspection();
	}

	using FNodeExecData = FPCGGraphExecutionInspection::FNodeExecutedNotificationData;

	const TSet<FNodeExecData> NodeStacks = Inspection.GetExecutedNodeStacks(Node);

	if (NodeStacks.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("No execution data for node '%s'. Inspection is now enabled - execute the graph instance via ExecuteGraphInstance and call this tool again."), 
			*Node->GetFName().ToString()));
		return {};
	}

	// Multiple stacks indicates node is part of a loop. Retrieving data for individual iterations is not yet supported.
	// @todo_pcg: Support selecting a specific loop iteration via a LoopIterationIndex parameter.
	if (NodeStacks.Num() > 1)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Node '%s' was executed %d times (inside a loop or instantiated multiple times). Per-iteration data retrieval is not yet supported."),
			*Node->GetFName().ToString(), NodeStacks.Num()));
		return {};
	}

	// Get the first stack and add the node and pin.
	FPCGStack QueryStack = NodeStacks.CreateConstIterator()->Stack;
	TArray<FPCGStackFrame>& StackFrames = QueryStack.GetStackFramesMutable();
	StackFrames.Reserve(StackFrames.Num() + 2);
	StackFrames.Emplace(Node);
	StackFrames.Emplace(OutputPin);

	TSharedPtr<FPCGInspectionData> InspectionData = Inspection.GetInspectionDataPtr(QueryStack);
	if (!InspectionData || !InspectionData->Data)
	{
		FString ErrorMessage = FString::Printf(TEXT("Node '%s' was executed but produced no data on pin '%s'."), *Node->GetFName().ToString(), *PinLabel);
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	const FPCGDataCollection& DataCollection = *InspectionData->Data;
	if (DataCollection.TaggedData.IsEmpty())
	{
		FString ErrorMessage = FString::Printf(TEXT("Node '%s' was executed but produced no data on pin '%s'."), *Node->GetFName().ToString(), *PinLabel);
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	// Serialize each data item with UPCGDataViewJsonConverter
	FInstancedStruct Params;
	Params.InitializeAs<FPCGDataViewJsonParameters>();
	FPCGDataViewJsonParameters& JsonParams = Params.GetMutable<FPCGDataViewJsonParameters>();
	JsonParams.AttributeLayout = EPCGDataViewAttributeLayout::ByElement;
	JsonParams.bPrettyJson = false;

	const UPCGDataViewJsonConverter* Converter = GetDefault<UPCGDataViewJsonConverter>();

	TArray<FString> SerializedItems;
	for (const FPCGTaggedData& TaggedData : DataCollection.TaggedData)
	{
		const UPCGData* Data = TaggedData.Data.Get();
		if (!Data)
		{
			continue;
		}

		FPCGDataView DataView;
		if (const UPCGDataViewData* DataViewData = Cast<UPCGDataViewData>(Data))
		{
			DataView = DataViewData->GetDataView();
		}
		else
		{
			DataView.ViewedData = Data;
		}

		// Configure selection based on AttributeName
		if (AttributeName.IsEmpty())
		{
			DataView.Selection.bAllAttributes = true;
		}
		else // Set the attribute explicitly
		{
			DataView.Selection.bAllAttributes = false;
			DataView.Selection.Attributes.Reset();
			DataView.Selection.Attributes.Add(FPCGAttributePropertySelector::CreateSelectorFromString<FPCGAttributePropertySelector>(AttributeName));
		}

		// Serialize to JSON string
		TValueOrError<FString, FText> JsonResult = Converter->SerializeToString(DataView, Params);
		if (JsonResult.HasError())
		{
			UKismetSystemLibrary::RaiseScriptError(JsonResult.GetError().ToString());
			return {};
		}

		FString JsonString = JsonResult.StealValue();

		// @todo_pcg: FPCGDataView/FPCGDataViewSelection could support element ranges natively.
		// Apply element range if needed
		if (StartIndex != 0 || EndIndex != -1)
		{
			TSharedPtr<FJsonObject> JsonObject = Json::ParseJson(JsonString);
			if (JsonObject)
			{
				const int32 NumElements = PCGHelpers::GetNumberOfElements(Data);

				if (StartIndex < 0 || StartIndex >= NumElements)
				{
					FString ErrorMessage = FString::Printf(TEXT("Error: StartIndex (%d) out of bounds. Data has %d elements."), StartIndex, NumElements);
					UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
					return {};
				}

				const int32 EffectiveEnd = (EndIndex == -1) ? NumElements : FMath::Min(EndIndex, NumElements);

				// Remove element_N fields outside [StartIndex, EffectiveEnd), preserving original numbering
				for (int32 i = 0; i < NumElements; ++i)
				{
					if (i < StartIndex || i >= EffectiveEnd)
					{
						JsonObject->RemoveField(FString::Printf(TEXT("element_%d"), i));
					}
				}

				JsonObject->SetNumberField(TEXT("totalElements"), NumElements);
				JsonString = Json::ToJsonString(JsonObject);
			}
			else
			{
				FString ErrorMessage = FString::Printf(TEXT("Error: Failed to parse serialized JSON for range filtering (StartIndex=%d, EndIndex=%d)."), StartIndex, EndIndex);
				UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
				return {};
			}
		}

		SerializedItems.Add(MoveTemp(JsonString));
	}

	if (SerializedItems.IsEmpty())
	{
		FString ErrorMessage = FString::Printf(TEXT("Node '%s' was executed but had no qualified data on pin '%s'."), *Node->GetFName().ToString(), *PinLabel);
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage);
		return {};
	}

	// Return single or by array
	if (SerializedItems.Num() == 1)
	{
		return SerializedItems[0];
	}

	return FString::Printf(TEXT("[%s]"), *FString::Join(SerializedItems, TEXT(",")));
}

// Comment Boxes ---------------------------------------------------------------------------------------------------
FString UPCGToolset::AddCommentBox(UPCGGraph* Graph, const TArray<UPCGNode*>& Nodes, const FString& Comment, FLinearColor Color)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return {};
	}

	if (Nodes.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No nodes to put in comment box");
		return {};
	}

	if (Algo::AnyOf(Nodes, [](const UPCGNode* Node){ return Node == nullptr; }))
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Nodes cannot contain null object.");
		return {};
	}

	// Check that the nodes belong to the graph.
	if (!Algo::AllOf(Nodes, [Graph](UPCGNode* Node) { return Node->GetGraph() == Graph; }))
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Nodes should all belong to the Graph");
		return {};
	}

	FIntRect CommentBox = PCGToolset::GetNodesBoundBox(Nodes);

	TArray<FPCGGraphCommentNodeData> CommentNodes = Graph->GetCommentNodes();
	FPCGGraphCommentNodeData& CommentAdded = CommentNodes.Emplace_GetRef(
		CommentBox.Min.X,
		CommentBox.Min.Y,
		CommentBox.Width(),
		CommentBox.Height(),
		Comment, 
		Color);
	CommentAdded.GUID = FGuid::NewGuid();

	FScopedTransaction Transaction(INVTEXT("AddCommentBoxToolCall"));
	Graph->Modify();
	Graph->SetCommentNodes(CommentNodes);
	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);

	return CommentAdded.GUID.ToString();
}

bool UPCGToolset::UpdateCommentBox(UPCGGraph* Graph, const FString& CommentId, const TArray<UPCGNode*>& Nodes, const FString& Comment,
                                        FLinearColor Color)
{
	using namespace PCGToolsetLibrary;
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	TArray<FPCGGraphCommentNodeData> CommentNodes = Graph->GetCommentNodes();
	FPCGGraphCommentNodeData* FoundComment = CommentNodes.FindByPredicate([&CommentId](const FPCGGraphCommentNodeData& Comment)
	{
		return Comment.GUID.ToString() == CommentId;
	});

	if (!FoundComment)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No Comment with the given id.");
		return false;
	}

	if (!Nodes.IsEmpty())
	{
		if (Algo::AnyOf(Nodes, [](const UPCGNode* Node) { return Node == nullptr; }))
		{
			UKismetSystemLibrary::RaiseScriptError("Error: Nodes cannot contain null object.");
			return false;
		}

		// Check that the nodes belong to the graph.
		if (!Algo::AllOf(Nodes, [Graph](UPCGNode* Node) { return Node->GetGraph() == Graph; }))
		{
			UKismetSystemLibrary::RaiseScriptError("Error: Nodes should all belong to the Graph");
			return false;
		}

		const FIntRect CommentBox = PCGToolset::GetNodesBoundBox(Nodes);

		FoundComment->NodePosX = CommentBox.Min.X;
		FoundComment->NodePosY = CommentBox.Min.Y;
		FoundComment->NodeWidth = CommentBox.Width();
		FoundComment->NodeHeight = CommentBox.Height();
	}

	FoundComment->CommentColor = Color;
	if (!Comment.IsEmpty())
	{
		FoundComment->NodeComment = Comment;
	}

	FScopedTransaction Transaction(INVTEXT("UpdateCommentBoxToolCall"));
	Graph->Modify();
	Graph->SetCommentNodes(CommentNodes);
	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	return true;
}

bool UPCGToolset::RemoveCommentBox(UPCGGraph* Graph, const FString& CommentId)
{
	if (!Graph)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: Graph not found");
		return false;
	}

	TArray<FPCGGraphCommentNodeData> CommentNodes = Graph->GetCommentNodes();
	const int32 NbRemoved = CommentNodes.RemoveAll([&CommentId](const FPCGGraphCommentNodeData& Comment)
	{
		return Comment.GUID.ToString() == CommentId;
	});

	if (NbRemoved == 0)
	{
		UKismetSystemLibrary::RaiseScriptError("Error: No Comment with the given id.");
		return false;
	}

	FScopedTransaction Transaction(INVTEXT("RemoveCommentBoxToolCall"));
	Graph->Modify();
	Graph->SetCommentNodes(CommentNodes);
	[[maybe_unused]] bool bDummy = Graph->MarkPackageDirty();
	Graph->ForceNotificationForEditor(EPCGChangeType::ExternalModification);
	return true;
}
