// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGToolset.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGVolume.h"
#include "Editor/IPCGEditorModule.h"
#include "Helpers/PCGHelpers.h"
#include "Subsystems/PCGSubsystem.h"

#include "Editor.h"
#include "EditorModeManager.h"
#include "InteractiveToolManager.h"
#include "LevelEditorSubsystem.h"
#include "PCGToolsetLibraryCore.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "ToolsetRegistry/ToolCallAsyncResultVoid.h"

namespace UPCGToolsetAsync::Private
{
	void TriggerSelectionUpdate()
	{
		if (GEditor && GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
		{
			// Since we might have been moving things around (component wise) we need to force a selection set change,
			// Which in turn will trigger a component visualizer update.
			if (UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet())
			{
				SelectionSet->OnChanged().Broadcast(SelectionSet);
			}
		}
	}
} // namespace UPCGToolsetAsync::Private

UPCGExecuteGraphInstanceAsyncResult* UPCGToolset::ExecuteGraphInstance(const APCGVolume* PCGVolume)
{
	UPCGExecuteGraphInstanceAsyncResult* AsyncResult = NewObject<UPCGExecuteGraphInstanceAsyncResult>();

	if (!PCGVolume)
	{
		AsyncResult->SetError(TEXT("Error: PCGVolume is null"));
		return AsyncResult;
	}

	UPCGComponent* Component = PCGVolume->PCGComponent;
	if (!Component)
	{
		AsyncResult->SetError(TEXT("Error: No PCG Component found"));
		return AsyncResult;
	}

	FPCGTaskId TaskId = Component->GenerateLocalGetTaskId(/*bForce=*/true);

	if (TaskId != InvalidPCGTaskId)
	{
		if (UPCGSubsystem* PCGSubsystem = Component->GetSubsystem())
		{
			PCGSubsystem->ScheduleGeneric([Component = TWeakObjectPtr<UPCGComponent>(Component), AsyncResult = TStrongObjectPtr(AsyncResult)]()
			{
				if (!Component.IsValid())
				{
					AsyncResult->SetError(TEXT("PCG Component was no longer valid."));
					return true;
				}

				const PCGUtils::FExtraCapture& Capture = Component->GetExecutionState().GetExtraCapture();
				const TMap<TWeakObjectPtr<const UPCGNode>, TArray<PCGUtils::FCapturedMessage>>& NodeToMessageMap = Capture.GetCapturedMessages();

				TArray<FPCGNodeExecutionMessage> Messages;
				for (const TPair<TWeakObjectPtr<const UPCGNode>, TArray<PCGUtils::FCapturedMessage>>& Pair : NodeToMessageMap)
				{
					const UPCGNode* Node = Pair.Key.Get();
					if (!Node)
					{
						continue;
					}

					const FString NodeName = Node->GetFName().ToString();
					for (const PCGUtils::FCapturedMessage& CapturedMessage : Pair.Value)
					{
						FPCGNodeExecutionMessage& Entry = Messages.AddDefaulted_GetRef();
						Entry.Message = CapturedMessage.Message;
						Entry.Severity = PCGToolsetLibrary::Graph::VerbosityToString(CapturedMessage.Verbosity);
						Entry.ReporterNode = NodeName;
					}
				}

				AsyncResult->SetValue(MoveTemp(Messages));
				return true;
			}, Component, {TaskId});

			return AsyncResult;
		}
	}

	AsyncResult->SetError(FString::Format(TEXT("Failed to call Execute on instance {0}."), { PCGVolume->GetName() }));
	return AsyncResult;
}

UToolCallAsyncResultVoid* UPCGToolset::DrawSpline(const FString& ActorLabel, const FString& ActorTag, bool bRedraw, bool bClosedSpline)
{
	const FEditorModeID EM_PCGEditorModeId = TEXT("EM_PCGEditorMode");
	ensure(IsInGameThread());

	PCGUtils::FScopedCallOutputDevice OutputDevice;
	PCGUtils::FScopedCall ScopedCall(nullptr, nullptr, OutputDevice);

	UEditorActorSubsystem* EditorActorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UEditorActorSubsystem>() : nullptr;

	if (!GEditor || !EditorActorSubsystem)
	{
		UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
		ReturnValue->SetError(TEXT("Redraw: Invalid engine state (unable to retrieve editor or editor actor subsystem)."));
		return ReturnValue;
	}

	if (bRedraw)
	{
		TArray<AActor*> AllActors = EditorActorSubsystem->GetAllLevelActors();
		AActor** FoundActor = AllActors.FindByPredicate([&ActorLabel](AActor* Actor) { return Actor->GetActorLabel() == ActorLabel; });
		if (!FoundActor)
		{
			UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
			ReturnValue->SetError(TEXT("Redraw: Actor not found."));
			return ReturnValue;
		}

		EditorActorSubsystem->SetSelectedLevelActors({*FoundActor});
	}
	else
	{
		EditorActorSubsystem->SelectNothing();
	}

	IConsoleObject* EnableToolConsoleObject = IConsoleManager::Get().FindConsoleObject((TEXT("pcg.EnableTool")));
	IConsoleCommand* EnableToolCommand = EnableToolConsoleObject ? EnableToolConsoleObject->AsCommand() : nullptr;
	FString ToolName = bClosedSpline ? "SplineSurfaceTool" : "SplineTool";
	const TArray<FString> EnableToolArgs = {ToolName};

	if (EnableToolCommand)
	{
		if (IPCGEditorModule* PCGEditorModule = IPCGEditorModule::Get())
		{
			// This actually starts the sequence of events. It must be done before getting the events if we want the tool
			//   manager to be the right one for the events.
			UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext(true).World() : nullptr;
			if (!EnableToolCommand->Execute(EnableToolArgs, EditorWorld, OutputDevice))
			{
				UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
				ReturnValue->SetError(TEXT("Fail to enable draw mode."));
				return ReturnValue;
			}

			TSharedPtr<FDelegateHandle> ToolEndedDelegateHandle = MakeShared<FDelegateHandle>();
			TSharedPtr<FDelegateHandle> ToolStartedDelegateHandle = MakeShared<FDelegateHandle>();
			FEditorModeTools& ModeTools = GLevelEditorModeTools();

			UEdMode* EditorMode = ModeTools.GetActiveScriptableMode(EM_PCGEditorModeId);
			UInteractiveToolManager* ToolManager = EditorMode ? EditorMode->GetToolManager() : nullptr;

			if (!ToolManager)
			{
				UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
				ReturnValue->SetError(TEXT("Fail to enable draw mode."));
				return ReturnValue;
			}

			UInteractiveToolManager::FToolManagerToolStartedSignature& OnEditorModeToolStarted = ToolManager->OnToolStarted;
			UInteractiveToolManager::FToolManagerToolCancelledSignature& OnEditorModeToolEnded = ToolManager->OnToolEndedWithStatus;

			// Temporary label of the created actor to make sure we do not post-process the wrong actor. 
			FGuid NewActorGuidName = FGuid::NewGuid();
			FString TemporaryUniqueActorLabel = "TempDrawSplineActor_" + NewActorGuidName.ToString();
			
			TObjectPtr<UToolCallAsyncResultVoid> AsyncResult = NewObject<UToolCallAsyncResultVoid>();
			auto ToolEndedLambda = [
				AsyncResult = TStrongObjectPtr(AsyncResult.Get()),
				ActorLabel = ActorLabel,
				TemporaryUniqueActorLabel,
				bRedraw,
				ActorTag = ActorTag,
				ToolEndedDelegateHandle = ToolEndedDelegateHandle,
				&OnEditorModeToolEnded]
			(UInteractiveToolManager* ToolManager, UInteractiveTool* InTool, EToolShutdownType ShutdownReason)
			{
				ensure(IsInGameThread());
				if (ToolEndedDelegateHandle && ToolEndedDelegateHandle->IsValid())
				{
					ToolEndedDelegateHandle->Reset();
					OnEditorModeToolEnded.Remove(*ToolEndedDelegateHandle);
				}

				if (ShutdownReason != EToolShutdownType::Cancel)
				{
					UEditorActorSubsystem* EditorActorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UEditorActorSubsystem>() : nullptr;

					TArray<AActor*> AllActors;
					if (EditorActorSubsystem)
					{
						AllActors = EditorActorSubsystem->GetAllLevelActors();
					}

					const FString& ActorLabelToSearch = bRedraw ? ActorLabel : TemporaryUniqueActorLabel;
					if (AActor** FoundActorPtr = AllActors.FindByPredicate([&ActorLabelToSearch](const AActor* Actor) { return Actor->GetActorLabel() == ActorLabelToSearch; }))
					{
						check(*FoundActorPtr);
						AActor* Actor = *FoundActorPtr;
						if (UPCGComponent* PCGComponent = Actor->FindComponentByClass<UPCGComponent>())
						{
							Actor->RemoveOwnedComponent(PCGComponent);
							PCGComponent->DestroyComponent();
						}
						
						if (!bRedraw)
						{
							Actor->SetActorLabel(ActorLabel);
						}
						
						if (!Actor->Tags.Contains(ActorTag))
						{
							Actor->Tags.Add(FName(ActorTag));
						}
						
						AsyncResult->SetCompleted();
					}
					else
					{
						const FString ErrorMessage = FString::Format(TEXT("Cannot tag new actor as it is not in the level: '{0}'."), {ActorLabelToSearch});
						AsyncResult->SetError(ErrorMessage);
					}
				}
				else
				{
					AsyncResult->SetError(TEXT("Tool mode was cancelled by the user."));
				}

				// Restore to default mode anyway at the end.
				GLevelEditorModeTools().ActivateMode(FEditorModeID("EM_Default"));

				// Trigger a selection update to make sure drawn splines are still properly visible
				UPCGToolsetAsync::Private::TriggerSelectionUpdate();
			};

			auto ToolStartedLambda = [
				ToolEndedLambda,
				AsyncResult = TStrongObjectPtr(AsyncResult.Get()),
				ToolEndedDelegateHandle = ToolEndedDelegateHandle,
				ToolStartedDelegateHandle = ToolStartedDelegateHandle,
				&OnEditorModeToolStarted,
				&OnEditorModeToolEnded,
				ActorLabel=TemporaryUniqueActorLabel]
			(UInteractiveToolManager* ToolManager, UInteractiveTool* InTool)
			{
				PCGUtils::FScopedCallOutputDevice OutputDevice;
				PCGUtils::FScopedCall ScopedCall(nullptr, nullptr, OutputDevice);

				ensure(IsInGameThread());
				if (ToolStartedDelegateHandle && ToolStartedDelegateHandle->IsValid())
				{
					ToolStartedDelegateHandle->Reset();
					OnEditorModeToolStarted.Remove(*ToolStartedDelegateHandle);
				}

				// Make sure the next sequence event is caught.
				*ToolEndedDelegateHandle = OnEditorModeToolEnded.AddLambda(ToolEndedLambda);

				IConsoleObject* SetToolSettingsConsoleObject = IConsoleManager::Get().FindConsoleObject((TEXT("pcg.tool.SetGraph")));

				IConsoleCommand* SetToolSettingsCommand = SetToolSettingsConsoleObject ? SetToolSettingsConsoleObject->AsCommand() : nullptr;
				const FString EmptyGraphPath{};
				const TArray<FString> SetToolSettingsArgs = {EmptyGraphPath, ActorLabel};
				UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext(true).World() : nullptr;
				if (!SetToolSettingsCommand || !SetToolSettingsCommand->Execute(SetToolSettingsArgs, EditorWorld, OutputDevice))
				{
					AsyncResult->SetError(TEXT("Fail to set draw tool settings."));
				}
			};
			*ToolStartedDelegateHandle = OnEditorModeToolStarted.AddLambda(ToolStartedLambda);

			return AsyncResult;
		}
		else
		{
			UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
			ReturnValue->SetError(TEXT("Fail to get PCGEditorModule."));
			return ReturnValue;
		}
	}

	UToolCallAsyncResultVoid* ReturnValue = NewObject<UToolCallAsyncResultVoid>();
	ReturnValue->SetError(TEXT("Unknown error."));
	return ReturnValue;
}
