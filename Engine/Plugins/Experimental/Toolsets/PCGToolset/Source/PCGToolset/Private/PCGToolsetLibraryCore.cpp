// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGToolsetLibraryCore.h"

#include "PCGToolsetModule.h"
#include "PCGToolsetCustomTypes.h"
#include "PCGToolsetSettings.h"

#include "PCGEdge.h"
#include "PCGGraph.h"
#include "PCGVolume.h"
#include "PCGSettings.h"
#include "PCGSubgraph.h"

#include "EngineUtils.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Helpers/PCGPropertyHelpers.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "ToolsetRegistry/ToolsetLibrary.h"

// Json helpers
TSharedPtr<FJsonObject> PCGToolsetLibrary::Json::ParseJson(const FString& JsonString)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
	FJsonSerializer::Deserialize(Reader, JsonObject);
	return JsonObject;
}

FString PCGToolsetLibrary::Json::ToJsonString(const TSharedPtr<FJsonObject>& JsonObject)
{
	FString Output;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Output);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
	return Output;
}

FString PCGToolsetLibrary::Json::ToJsonString(const TArray<TSharedPtr<FJsonValue>>& JsonArray)
{
	FString Output;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Output);
	FJsonSerializer::Serialize(JsonArray, Writer);
	return Output;
}

namespace
{
	// Builds a bag containing each Property whose value in OverrideContainer differs from DefaultContainer.
	// Property offsets are used for both containers, which is safe when both point at memory of the same struct layout
	// (UPropertyBag::GetOrCreateFromDescs caches by desc list, so matching descs yield matching offsets).
	FInstancedPropertyBag BuildOverrideBag(TArrayView<FProperty*> Properties, const void* OverrideContainer, const void* DefaultContainer)
	{
		TArray<FProperty*> Overridden;
		TArray<FPropertyBagPropertyDesc> Descs;
		for (FProperty* Property : Properties)
		{
			const void* OverridePtr = Property->ContainerPtrToValuePtr<void>(OverrideContainer);
			const void* DefaultPtr = Property->ContainerPtrToValuePtr<void>(DefaultContainer);
			if (Property->Identical(OverridePtr, DefaultPtr))
			{
				continue;
			}

			Overridden.Add(Property);
			Descs.Emplace(Property->GetFName(), Property);
		}

		FInstancedPropertyBag Result;
		if (Descs.IsEmpty())
		{
			return Result;
		}

		Result.AddProperties(Descs);
		if (!Result.GetPropertyBagStruct())
		{
			return {};
		}

		uint8* ResultMemory = Result.GetMutableValue().GetMemory();
		for (FProperty* SourceProperty : Overridden)
		{
			const FPropertyBagPropertyDesc* ResultDesc = Result.FindPropertyDescByName(SourceProperty->GetFName());
			if (!ResultDesc || !ResultDesc->CachedProperty)
			{
				continue;
			}

			const void* SourcePtr = SourceProperty->ContainerPtrToValuePtr<void>(OverrideContainer);
			void* DestPtr = ResultDesc->CachedProperty->ContainerPtrToValuePtr<void>(ResultMemory);
			SourceProperty->CopyCompleteValue(DestPtr, SourcePtr);
		}

		return Result;
	}
}

// Build helpers
FInstancedPropertyBag PCGToolsetLibrary::Graph::BuildFilteredBag(const FInstancedPropertyBag& SourceBag)
{
	const UPropertyBag* BagStruct = SourceBag.GetPropertyBagStruct();
	if (!BagStruct)
	{
		return {};
	}

	TArray<FPropertyBagPropertyDesc> FilteredDescs;
	for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
	{
		if (!Desc.CachedProperty)
		{
			continue;
		}

		if (IsPrimitiveInternalParam(Desc.CachedProperty))
		{
			continue;
		}

		FilteredDescs.Add(Desc);
	}

	if (FilteredDescs.IsEmpty())
	{
		return {};
	}

	FInstancedPropertyBag Result;
	Result.AddProperties(FilteredDescs);
	Result.CopyMatchingValuesByName(SourceBag);
	return Result;
}

void PCGToolsetLibrary::Graph::EnablePropertyOverrides(UPCGGraphInstance* GraphInstance, const TSharedPtr<FJsonObject>& JsonKeys)
{
	if (!GraphInstance || !JsonKeys.IsValid())
	{
		return;
	}

	FInstancedPropertyBag* Parameters = GraphInstance->GetMutableUserParametersStruct_Unsafe();
	if (!Parameters)
	{
		return;
	}

	for (const auto& Pair : JsonKeys->Values)
	{
		const FName ParamName(*FString(Pair.Key).TrimStartAndEnd());
		if (const FPropertyBagPropertyDesc* PropertyDesc = Parameters->FindPropertyDescByName(ParamName))
		{
			GraphInstance->UpdatePropertyOverride(PropertyDesc->CachedProperty, true);
		}
	}
}

bool PCGToolsetLibrary::Graph::IsPrimitiveInternalParam(const FProperty* Property)
{
	return Constants::PrimitiveInternalParamNames.Contains(Property->GetAuthoredName());
}

void PCGToolsetLibrary::Graph::RaiseScopedErrors(const PCGUtils::FScopedCall& ScopedCall)
{
	// We need a copy because emitting the errors here will add them again in the log.
	TArray<PCGUtils::FCapturedMessage> CapturedMessagesCopy = ScopedCall.CapturedMessages;
	for (const PCGUtils::FCapturedMessage& Message : CapturedMessagesCopy)
	{
		if (Message.Verbosity == ELogVerbosity::Error || Message.Verbosity == ELogVerbosity::Warning)
		{
			UKismetSystemLibrary::RaiseScriptError(Message.Message);
		}
	}
}

FString PCGToolsetLibrary::Graph::VerbosityToString(ELogVerbosity::Type Verbosity)
{
	switch (Verbosity)
	{
		case ELogVerbosity::Error:
			return TEXT("Error");
		case ELogVerbosity::Warning:
			return TEXT("Warning");
		default:
			return TEXT("Feedback");
	}
}

// Graph Params
FInstancedPropertyBag PCGToolsetLibrary::Graph::GetGraphParams(const UPCGGraph* Graph)
{
	const FInstancedPropertyBag* PropertyBag = Graph ? Graph->GetUserParametersStruct() : nullptr;

	if (!PropertyBag || !PropertyBag->IsValid())
	{
		return {};
	}

	return BuildFilteredBag(*PropertyBag);
}

FInstancedPropertyBag PCGToolsetLibrary::Graph::GetSubgraphNodeParamOverrides(const UPCGSubgraphSettings* SubgraphSettings, const UPCGGraph* Subgraph)
{
	if (!SubgraphSettings)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Invalid subgraph settings"));
		return {};
	}

	const TObjectPtr<UPCGGraphInstance> SubgraphInstance = SubgraphSettings->SubgraphInstance;
	if (!SubgraphInstance)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Subgraph node has no subgraph instance"));
		return {};
	}

	const FInstancedPropertyBag* DefaultBag = Subgraph ? Subgraph->GetUserParametersStruct() : nullptr;
	const FInstancedPropertyBag* OverridesBag = SubgraphInstance->GetUserParametersStruct();
	if (!DefaultBag || !DefaultBag->IsValid() || !OverridesBag || !OverridesBag->IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: Unable to retrieve subgraph node param overrides"));
		return {};
	}

	const TConstArrayView<FPropertyBagPropertyDesc> DefaultDescs = DefaultBag->GetPropertyBagStruct()->GetPropertyDescs();
	const TConstArrayView<FPropertyBagPropertyDesc> OverrideDescs = OverridesBag->GetPropertyBagStruct()->GetPropertyDescs();
	if (DefaultDescs.Num() != OverrideDescs.Num())
	{
		return {};
	}

	TArray<FProperty*> OverrideProperties;
	for (int32 i = 0; i < OverrideDescs.Num(); ++i)
	{
		if (OverrideDescs[i].Name != DefaultDescs[i].Name)
		{
			return {};
		}

		FProperty* Property = const_cast<FProperty*>(OverrideDescs[i].CachedProperty);
		if (!Property || IsPrimitiveInternalParam(Property))
		{
			continue;
		}

		OverrideProperties.Add(Property);
	}

	return BuildOverrideBag(OverrideProperties, OverridesBag->GetValue().GetMemory(), DefaultBag->GetValue().GetMemory());
}

bool PCGToolsetLibrary::Graph::SetGraphInstanceParams(UPCGGraphInstance* GraphInstance, const FString& JsonParams)
{
	if (!GraphInstance)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Error: GraphInstance is null"));
		return false;
	}

	TSharedPtr<FJsonObject> ParsedJson = Json::ParseJson(JsonParams);
	if (!ParsedJson.IsValid())
	{
		return false;
	}

	EnablePropertyOverrides(GraphInstance, ParsedJson);

	// Wrap as {"ParametersOverrides": {"Parameters": <parsed>}} so SetObjectProperties
	// descends into the graph instance's parameter bag.
	TSharedPtr<FJsonObject> Overrides = MakeShared<FJsonObject>();
	Overrides->SetObjectField(TEXT("Parameters"), ParsedJson);
	TSharedPtr<FJsonObject> Wrapped = MakeShared<FJsonObject>();
	Wrapped->SetObjectField(TEXT("ParametersOverrides"), Overrides);

	return UToolsetLibrary::SetObjectProperties(GraphInstance, Json::ToJsonString(Wrapped), EBypassContainerCheck::Yes);
}

FPCGNodeInfo PCGToolsetLibrary::Graph::GetNodeInfo(const UPCGNode* Node)
{
	FPCGNodeInfo Info;
	if (!Node || !Node->GetSettings())
	{
		return Info;
	}

	UPCGSettings* Settings = Node->GetSettings();
	if (Settings->IsA<UPCGSubgraphSettings>())
	{
		UPCGSubgraphSettings* SubgraphSettings = Cast<UPCGSubgraphSettings>(Settings);
		const UPCGGraph* Subgraph = SubgraphSettings->GetSubgraph();
		if (Subgraph)
		{
			Info.NodeType = FString::Format(TEXT("Subgraph({0})"), {Subgraph->GetFName().ToString()});
			Info.ParamOverrides = GetSubgraphNodeParamOverrides(SubgraphSettings, Subgraph);
		}
		else
		{
			Info.NodeType = TEXT("Subgraph");
		}
	}
	else
	{
		Info.NodeType = Node->GetDefaultTitle().ToString();
		UClass* SettingsClass = Settings->GetClass();
		TArray<FProperty*> Properties = GetNodePropertiesFromSettings(SettingsClass);
		const UPCGSettings* DefaultSettings = GetRealDefaultObject(SettingsClass);
		Info.ParamOverrides = BuildOverrideBag(Properties, Settings, DefaultSettings);
	}

	Info.Name = Node->GetFName().ToString();
	Info.Path = Node->GetPathName();
	Info.Position = FString::Format(TEXT("({0}, {1})"), {Node->PositionX, Node->PositionY});
	Info.Title = Node->GetNodeTitle(EPCGNodeTitleType::ListView).ToString();
	Info.bEnabled = Node->GetSettingsInterface()->bEnabled;
	Info.Comment = Node->NodeComment;

	return Info;
}

// Nodes
TArray<FPCGNodeInfo> PCGToolsetLibrary::Graph::GetGraphNodesInfo(const UPCGGraph* Graph)
{
	TArray<FPCGNodeInfo> Nodes;
	if (!Graph)
	{
		return Nodes;
	}

	TArray<UPCGNode*> GraphNodes = Graph->GetNodes();
	GraphNodes.Add(Graph->GetInputNode());
	GraphNodes.Add(Graph->GetOutputNode());
	for (UPCGNode* GraphNode : GraphNodes)
	{
		if (GraphNode && GraphNode->GetSettings())
		{
			Nodes.Add(GetNodeInfo(GraphNode));
		}
	}

	return Nodes;
}


TArray<FProperty*> PCGToolsetLibrary::Graph::GetNodePropertiesFromSettings(TSubclassOf<UPCGSettings> InSettingsClass)
{
	TArray<FProperty*> Properties;

	const UStruct* StopClass = UPCGSettings::StaticClass();
	for (const UStruct* CurClass = InSettingsClass; CurClass != nullptr && (StopClass == nullptr || CurClass != StopClass); CurClass = CurClass->GetSuperStruct())
	{
		for (TFieldIterator<FProperty> InputIt(CurClass, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated); InputIt; ++InputIt)
		{
			FProperty* Property = *InputIt;
			if (!Property)
			{
				continue;
			}

			if (IsPrimitiveInternalParam(Property))
			{
				UE_LOG(LogPCGToolset, Log, TEXT("Skipping internal primitive param %s"), *Property->GetFName().ToString());
				continue;
			}

			if (!Property->HasAnyPropertyFlags(CPF_Edit) || Property->HasAnyPropertyFlags(CPF_AdvancedDisplay) || Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGSettings, Seed))
			{
				UE_LOG(LogPCGToolset, Log, TEXT("Skipping property %s for node settings %s"), *Property->GetFName().ToString(), *InSettingsClass->GetAuthoredName())
				continue;
			}

			Properties.Add(Property);
		}
	}
	return Properties;
}

const TMap<FName, UPCGSettings*>& PCGToolsetLibrary::Graph::GetNodeNameToSettingsMap()
{
	ensure(IsInGameThread());
	static TMap<FName, UPCGSettings*> NodeNameToSettingsMap;

	if (NodeNameToSettingsMap.IsEmpty())
	{
		TArray<UClass*> PCGSettingsClasses;
		GetDerivedClasses(UPCGSettings::StaticClass(), PCGSettingsClasses, true);
		TMap<FName, TSubclassOf<UPCGSettings>> SettingsMap;
		UPackage* PCGPackage = FindPackage(nullptr, TEXT("/Script/PCG"));

		for (UClass* PCGSettingClass : PCGSettingsClasses)
		{
			if (PCGSettingClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden))
			{
				continue;
			}

			if (!PCGSettingClass->IsInPackage(PCGPackage))
			{
				continue;
			}

			UPCGSettings* PCGSettings = GetRealDefaultObject(PCGSettingClass);
			if (!PCGSettings || !PCGSettings->bExposeToLibrary)
			{
				continue;
			}

			NodeNameToSettingsMap.Add(FName(*PCGSettings->GetDefaultNodeTitle().ToString()), PCGSettings);
		}
	}

	return NodeNameToSettingsMap;
}

// Pins
TArray<FPCGPinInfo> PCGToolsetLibrary::Graph::GetNodePinsSchema(const TArray<FPCGPinProperties>& PinProperties)
{
	TArray<FPCGPinInfo> PinInfos;
	Algo::Transform(PinProperties, PinInfos, [](const FPCGPinProperties& Property)
	{
		FPCGPinInfo Info;
		Info.Name = Property.Label.ToString();
		Info.Type = Property.AllowedTypes.ToDisplayText().ToString();
		Info.Description = Property.Tooltip.ToString();
		return Info;
	});

	return PinInfos;
}

// Edges
TArray<FPCGEdgeInfo> PCGToolsetLibrary::Graph::GetGraphEdges(const UPCGGraph* Graph)
{
	if (!Graph)
	{
		return {};
	}

	const TArray<UPCGEdge*> Edges = Graph->GetAllEdges();
	TArray<FPCGEdgeInfo> EdgeInfos;
	Algo::Transform(Edges, EdgeInfos, [](const UPCGEdge* Edge)
	{
		check(Edge)
		FPCGEdgeInfo Info;
		Info.SrcNode = Edge->GetInputNode()->GetFName().ToString();
		Info.SrcPin = Edge->GetInputPinLabel().ToString();
		Info.DestNode = Edge->GetOutputNode()->GetFName().ToString();
		Info.DestPin = Edge->GetOutputPinLabel().ToString();
		return Info;
	});

	return EdgeInfos;
}

// Default Object
UPCGSettings* PCGToolsetLibrary::Graph::GetRealDefaultObject(UClass* PCGSettingClass)
{
	if (!PCGSettingClass)
	{
		return nullptr;
	}

	// Do not use classes default object since the default value could be wrong.
	static TMap<UClass*, TStrongObjectPtr<UPCGSettings>> DefaultObjectMap;
	TStrongObjectPtr<UPCGSettings>& FoundDefaultObject = DefaultObjectMap.FindOrAdd(PCGSettingClass);
	if (FoundDefaultObject == nullptr)
	{
		FoundDefaultObject.Reset(NewObject<UPCGSettings>(GetTransientPackage(), PCGSettingClass));
	}

	return FoundDefaultObject.Get();
}

// Constants
TSet<FName> PCGToolsetLibrary::Constants::GetSubgraphDirectories()
{
	const UPCGToolsetSettings* Settings = GetDefault<UPCGToolsetSettings>();
	TSet<FName> Directories;
	for (const FDirectoryPath& DirPath : Settings->SubgraphDirectories)
	{
		Directories.Add(FName(*DirPath.Path));
	}

	return Directories;
}

TSet<FName> PCGToolsetLibrary::Constants::GetExamplesDirectories()
{
	const UPCGToolsetSettings* Settings = GetDefault<UPCGToolsetSettings>();
	TSet<FName> Directories;
	for (const FDirectoryPath& DirPath : Settings->ExampleGraphDirectories)
	{
		Directories.Add(FName(*DirPath.Path));
	}

	return Directories;
}

TSet<FName> PCGToolsetLibrary::Constants::GetInstantGraphDirectories()
{
	const UPCGToolsetSettings* Settings = GetDefault<UPCGToolsetSettings>();
	TSet<FName> Directories;
	for (const FDirectoryPath& DirPath : Settings->InstantGraphDirectories)
	{
		Directories.Add(FName(*DirPath.Path));
	}

	return Directories;
}

// Examples
TArray<FString> PCGToolsetLibrary::Graph::FindGraphPaths(const TSet<FName>& PackagePaths, TFunctionRef<bool(const FString& /*PathName*/)> PathPredicate)
{
	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	for (const FName Directory : PackagePaths)
	{
		Filter.PackagePaths.Add(Directory);
	}

	Filter.ClassPaths.Add(UPCGGraphInterface::StaticClass()->GetClassPathName());

	const IAssetRegistry& Registry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TArray<FAssetData> AssetData;
	Registry.GetAssets(Filter, AssetData);

	TArray<FString> Paths;
	for (const FAssetData& Data : AssetData)
	{
		if (Data.IsRedirector())
		{
			continue;
		}

		// bIsTemplate is AssetRegistrySearchable on UPCGGraph, so we can filter without loading.
		bool bIsTemplate = false;
		Data.GetTagValue(GET_MEMBER_NAME_CHECKED(UPCGGraph, bIsTemplate), bIsTemplate);
		if (bIsTemplate)
		{
			continue;
		}

		const FString PathName = Data.GetSoftObjectPath().ToString();
		if (!PathPredicate(PathName))
		{
			continue;
		}

		Paths.Add(PathName);
	}
	return Paths;
}
