// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGToolsetModule.h"

#include "PCGAttributePropertySelectorConverter.h"
#include "PCGToolset.h"
#include "PCGSpatialToolset.h"
#include "PCGToolsetSkills.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ToolsetRegistry/AgentSkill.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

DEFINE_LOG_CATEGORY(LogPCGToolset);

#define LOCTEXT_NAMESPACE "FPCGToolsetModule"

void FPCGToolsetModule::StartupModule()
{
	UToolsetRegistry::RegisterToolsetClass(UPCGToolset::StaticClass());
	UToolsetRegistry::RegisterToolsetClass(UPCGSpatialToolset::StaticClass());

	SelectorConverter = MakeShared<UE::PCGToolset::FPCGAttributePropertySelectorConverter>();
	TValueOrError<TObjectPtr<UToolsetRegistrySubsystem>, FString> Subsystem = UToolsetRegistrySubsystem::Get(TEXT("PCGToolset"));
	if (Subsystem.HasValue())
	{
		Subsystem.GetValue()->ToolsetRegistry.RegisterConverter(SelectorConverter);
	}
}

void FPCGToolsetModule::ShutdownModule()
{
	if (SelectorConverter.IsValid())
	{
		TValueOrError<TObjectPtr<UToolsetRegistrySubsystem>, FString> Subsystem = UToolsetRegistrySubsystem::Get();
		if (Subsystem.HasValue())
		{
			Subsystem.GetValue()->ToolsetRegistry.UnregisterConverter(SelectorConverter);
		}

		SelectorConverter.Reset();
	}

	UToolsetRegistry::UnregisterToolsetClass(UPCGToolset::StaticClass());
	UToolsetRegistry::UnregisterToolsetClass(UPCGSpatialToolset::StaticClass());
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPCGToolsetModule, PCGToolset)