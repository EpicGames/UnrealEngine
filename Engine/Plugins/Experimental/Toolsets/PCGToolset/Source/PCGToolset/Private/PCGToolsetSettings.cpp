// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGToolsetSettings.h"

#define LOCTEXT_NAMESPACE "PCGToolsetSettings"

UPCGToolsetSettings::UPCGToolsetSettings()
{
	// Set default directories
	SubgraphDirectories.Add(FDirectoryPath{TEXT("/PCGPrimitives/Primitives")});
	ExampleGraphDirectories.Add(FDirectoryPath{TEXT("/PCGPrimitives/Examples")});
	InstantGraphDirectories.Add(FDirectoryPath{TEXT("/PCGPrimitives/Instants")});
}

FName UPCGToolsetSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UPCGToolsetSettings::GetSectionText() const
{
	return LOCTEXT("SectionText", "PCG Toolset");
}

FText UPCGToolsetSettings::GetSectionDescription() const
{
	return LOCTEXT("SectionDescription", "Configure AI-assisted PCG graph generation settings");
}
#endif

#undef LOCTEXT_NAMESPACE
