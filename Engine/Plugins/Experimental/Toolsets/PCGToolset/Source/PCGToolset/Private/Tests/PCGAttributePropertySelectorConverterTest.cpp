// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/PCGAttributePropertySelectorConverterTest.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"
#include "PCGAttributePropertySelectorConverter.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/UnrealType.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace SelectorTestHelpers
{
	FStructProperty* GetSelectorProperty()
	{
		return CastField<FStructProperty>(
			FPCGAttributeSelectorTestHolder::StaticStruct()->FindPropertyByName(TEXT("Selector")));
	}

	FProperty* GetPlainStringProperty()
	{
		return FPCGAttributeSelectorTestHolder::StaticStruct()->FindPropertyByName(TEXT("PlainString"));
	}
}

BEGIN_DEFINE_SPEC(FPCGAttributePropertySelectorConverterSpec,
	"AI.Toolsets.PCGToolset.Converters.AttributeSelector",
	PCGToolsetTest::Flags)

	UE::PCGToolset::FPCGAttributePropertySelectorConverter Converter;

END_DEFINE_SPEC(FPCGAttributePropertySelectorConverterSpec)

void FPCGAttributePropertySelectorConverterSpec::Define()
{
	Describe(TEXT("Round-trip"), [this]()
	{
		It(TEXT("preserves DSL selectors through PropertyToJsonData and JsonDataToProperty"), [this]()
		{
			const TArray<FString> Cases = {
				TEXT("$Density"),
				TEXT("MyAttr"),
				TEXT("MyAttr.X.Y"),
				TEXT("@Elements.MyAttr"),
			};

			FStructProperty* SelectorProp = SelectorTestHelpers::GetSelectorProperty();
			if (!TestNotNull(TEXT("Selector property reflected"), SelectorProp))
			{
				return;
			}

			for (const FString& Input : Cases)
			{
				FPCGAttributeSelectorTestHolder Source;
				Source.Selector.Update(FStringView(Input));
				const FString SourceCanonical = Source.Selector.ToString();

				const TSharedPtr<FJsonValue> Json = Converter.PropertyToJsonData(
					SelectorProp, SelectorProp->ContainerPtrToValuePtr<void>(&Source));
				if (!TestTrue(TEXT("PropertyToJsonData returned a value"), Json.IsValid()))
				{
					continue;
				}

				FString JsonString;
				TestTrue(TEXT("JSON value is a string"), Json->TryGetString(JsonString));
				TestEqual(*FString::Printf(TEXT("Serialized form for '%s'"), *Input),
					JsonString, SourceCanonical);

				FPCGAttributeSelectorTestHolder Sink;
				const bool bParsed = Converter.JsonDataToProperty(
					Json, SelectorProp, SelectorProp->ContainerPtrToValuePtr<void>(&Sink), nullptr);
				TestTrue(*FString::Printf(TEXT("JsonDataToProperty parsed '%s'"), *Input), bParsed);
				TestEqual(*FString::Printf(TEXT("Round-tripped '%s'"), *Input),
					Sink.Selector.ToString(), SourceCanonical);
			}
		});
	});

	Describe(TEXT("Schema and CanConvertProperty"), [this]()
	{
		It(TEXT("emits a string schema with DSL examples"), [this]()
		{
			FStructProperty* SelectorProp = SelectorTestHelpers::GetSelectorProperty();
			if (!TestNotNull(TEXT("Selector property reflected"), SelectorProp))
			{
				return;
			}

			const TSharedPtr<FJsonObject> Schema = Converter.PropertyToJsonSchema(SelectorProp);
			if (!TestTrue(TEXT("Schema returned"), Schema.IsValid()))
			{
				return;
			}

			TestEqual(TEXT("Schema type"), Schema->GetStringField(TEXT("type")), TEXT("string"));

			FString Description;
			TestTrue(TEXT("Schema has a description"),
				Schema->TryGetStringField(TEXT("description"), Description));
			TestTrue(TEXT("Description hints at '$' qualifier"), Description.Contains(TEXT("$")));
			TestTrue(TEXT("Description hints at '@' qualifier"), Description.Contains(TEXT("@")));
		});

		It(TEXT("matches selector struct properties and ignores plain strings"), [this]()
		{
			FStructProperty* SelectorProp = SelectorTestHelpers::GetSelectorProperty();
			FProperty* StringProp = SelectorTestHelpers::GetPlainStringProperty();
			if (!TestNotNull(TEXT("Selector property reflected"), SelectorProp) ||
				!TestNotNull(TEXT("String property reflected"), StringProp))
			{
				return;
			}

			TestTrue(TEXT("CanConvertProperty(selector)"), Converter.CanConvertProperty(SelectorProp));
			TestFalse(TEXT("CanConvertProperty(FString)"), Converter.CanConvertProperty(StringProp));
		});
	});

	Describe(TEXT("JsonDataToProperty rejects malformed input"), [this]()
	{
		It(TEXT("returns false on non-string JSON values"), [this]()
		{
			FStructProperty* SelectorProp = SelectorTestHelpers::GetSelectorProperty();
			if (!TestNotNull(TEXT("Selector property reflected"), SelectorProp))
			{
				return;
			}

			struct FCase { const TCHAR* Label; TSharedPtr<FJsonValue> Value; };
			const TArray<FCase> Cases = {
				{ TEXT("null"),   TSharedPtr<FJsonValue>() },
				{ TEXT("object"), MakeShared<FJsonValueObject>(MakeShared<FJsonObject>()) },
				{ TEXT("array"),  MakeShared<FJsonValueArray>(TArray<TSharedPtr<FJsonValue>>()) },
			};

			for (const FCase& Case : Cases)
			{
				FPCGAttributeSelectorTestHolder Sink;
				const bool bParsed = Converter.JsonDataToProperty(
					Case.Value, SelectorProp, SelectorProp->ContainerPtrToValuePtr<void>(&Sink), nullptr);
				TestFalse(*FString::Printf(TEXT("Rejects %s value"), Case.Label), bParsed);
			}
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
