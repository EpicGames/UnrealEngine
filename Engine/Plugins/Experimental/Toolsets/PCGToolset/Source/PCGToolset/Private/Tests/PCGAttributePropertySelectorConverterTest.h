// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "PCGAttributePropertySelectorConverterTest.generated.h"

// Test-only USTRUCT used to surface a real FProperty for the selector to the converter.
USTRUCT()
struct FPCGAttributeSelectorTestHolder
{
	GENERATED_BODY()

	UPROPERTY()
	FPCGAttributePropertyInputSelector Selector;

	UPROPERTY()
	FString PlainString;
};
