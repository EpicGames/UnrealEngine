// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "PCGToolset.h"
#include "Elements/PCGAttributeNoise.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FPCGToolsetCommentBoxSpec, "AI.Toolsets.PCGToolset.CommentBox", PCGToolsetTest::Flags)
	PCG_TEST_EXCEPTION_HELPERS()

	TStrongObjectPtr<UPCGGraph> Graph;
	TStrongObjectPtr<UPCGNode> SeedNode;
END_DEFINE_SPEC(FPCGToolsetCommentBoxSpec)

void FPCGToolsetCommentBoxSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
		Graph.Reset(PCGToolsetTest::MakeTransientGraph());
		UPCGSettings* IgnoredDefaults = nullptr;
		SeedNode.Reset(Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults));
	});

	AfterEach([this]()
	{
		Graph.Reset();
		SeedNode.Reset();
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("AddCommentBox"), [this]()
	{
		It(TEXT("returns a guid that exists on the graph"), [this]()
		{
			FString CommentId;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				CommentId = UPCGToolset::AddCommentBox(Graph.Get(), {SeedNode.Get()}, TEXT("first"));
			});
			TestFalse(TEXT("AddCommentBox returned an id"), CommentId.IsEmpty());

			const TArray<FPCGGraphCommentNodeData>& Comments = Graph->GetCommentNodes();
			const bool bFound = Comments.ContainsByPredicate(
				[&](const FPCGGraphCommentNodeData& C) { return C.GUID.ToString() == CommentId; });
			TestTrue(TEXT("Graph contains the new comment box"), bFound);
		});
	});

	Describe(TEXT("UpdateCommentBox"), [this]()
	{
		It(TEXT("changes the comment text"), [this]()
		{
			FString CommentId;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				CommentId = UPCGToolset::AddCommentBox(Graph.Get(), {SeedNode.Get()}, TEXT("old"));
			});
			if (!TestFalse(TEXT("AddCommentBox returned an id"), CommentId.IsEmpty()))
			{
				return;
			}

			bool bUpdated = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bUpdated = UPCGToolset::UpdateCommentBox(Graph.Get(), CommentId, {}, TEXT("new"));
			});
			TestTrue(TEXT("UpdateCommentBox succeeded"), bUpdated);

			const TArray<FPCGGraphCommentNodeData>& Comments = Graph->GetCommentNodes();
			const FPCGGraphCommentNodeData* Found = Comments.FindByPredicate(
				[&](const FPCGGraphCommentNodeData& C) { return C.GUID.ToString() == CommentId; });
			if (TestNotNull(TEXT("Comment still on graph"), Found))
			{
				TestEqual(TEXT("Comment text"), Found->NodeComment, TEXT("new"));
			}
		});

		It(TEXT("rejects a non-existent comment id"), [this]()
		{
			AddExpectedErrorPlain(TEXT("No Comment with the given id"));
			bool bUpdated = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bUpdated = UPCGToolset::UpdateCommentBox(Graph.Get(),
					TEXT("00000000-0000-0000-0000-000000000000"), {}, TEXT("new"));
			});
			TestFalse(TEXT("UpdateCommentBox rejects unknown id"), bUpdated);
			ExpectExceptionContains(TEXT("No Comment with the given id"));
		});
	});

	Describe(TEXT("RemoveCommentBox"), [this]()
	{
		It(TEXT("removes a comment box from the graph"), [this]()
		{
			FString CommentId;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				CommentId = UPCGToolset::AddCommentBox(Graph.Get(), {SeedNode.Get()}, TEXT("doomed"));
			});
			if (!TestFalse(TEXT("AddCommentBox returned an id"), CommentId.IsEmpty()))
			{
				return;
			}

			bool bRemoved = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bRemoved = UPCGToolset::RemoveCommentBox(Graph.Get(), CommentId);
			});
			TestTrue(TEXT("RemoveCommentBox succeeded"), bRemoved);

			const TArray<FPCGGraphCommentNodeData>& Comments = Graph->GetCommentNodes();
			const bool bStillThere = Comments.ContainsByPredicate(
				[&](const FPCGGraphCommentNodeData& C) { return C.GUID.ToString() == CommentId; });
			TestFalse(TEXT("Comment removed from graph"), bStillThere);
		});

		It(TEXT("rejects a non-existent comment id"), [this]()
		{
			AddExpectedErrorPlain(TEXT("No Comment with the given id"));
			bool bRemoved = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bRemoved = UPCGToolset::RemoveCommentBox(Graph.Get(),
					TEXT("00000000-0000-0000-0000-000000000000"));
			});
			TestFalse(TEXT("RemoveCommentBox rejects unknown id"), bRemoved);
			ExpectExceptionContains(TEXT("No Comment with the given id"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
