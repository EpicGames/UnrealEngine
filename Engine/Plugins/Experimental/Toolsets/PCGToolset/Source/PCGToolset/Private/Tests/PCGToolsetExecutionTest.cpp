// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Ticker.h"
#include "Editor.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Misc/AutomationTest.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGGraphExecutionInspection.h"
#include "PCGNode.h"
#include "PCGSettings.h"
#include "PCGToolset.h"
#include "PCGToolsetCustomTypes.h"
#include "PCGVolume.h"
#include "Elements/PCGAttributeNoise.h"
#include "Elements/PCGCreatePointsGrid.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace ExecutionTestHelpers
{
	// Drives the editor world + core ticker until Result->bIsComplete or timeout.
	bool WaitForAsyncResult(UToolCallAsyncResult* Result, UWorld* World, double TimeoutSec = 30.0)
	{
		if (!Result)
		{
			return false;
		}
		const double StartTime = FPlatformTime::Seconds();
		constexpr float DeltaTime = 1.0f / 30.0f;
		while (!Result->bIsComplete)
		{
			FTSTicker::GetCoreTicker().Tick(DeltaTime);
			if (World)
			{
				World->Tick(LEVELTICK_All, DeltaTime);
			}
			if (FPlatformTime::Seconds() - StartTime > TimeoutSec)
			{
				return false;
			}
		}
		return true;
	}

	/* GetNodeDataView serializes per-element fields as "element_0", "element_1", ... so the
	 * number of elements equals the count of those keys.
	 */
	int32 CountElementKeys(const FString& Json)
	{
		int32 Count = 0;
		int32 SearchIdx = 0;
		const FString Marker = TEXT("\"element_");
		while ((SearchIdx = Json.Find(Marker, ESearchCase::CaseSensitive,
			ESearchDir::FromStart, SearchIdx)) != INDEX_NONE)
		{
			++Count;
			SearchIdx += Marker.Len();
		}
		return Count;
	}
}

BEGIN_DEFINE_SPEC(FPCGToolsetExecutionSpec, "AI.Toolsets.PCGToolset.Execution",
	PCGToolsetTest::Flags)
	PCG_TEST_EXCEPTION_HELPERS()

	TStrongObjectPtr<UWorld> World;
	TStrongObjectPtr<UPCGGraph> Graph;
	TStrongObjectPtr<UPCGNode> GridNode;
	TStrongObjectPtr<UPCGNode> NoiseNode;
	TStrongObjectPtr<APCGVolume> Volume;
END_DEFINE_SPEC(FPCGToolsetExecutionSpec)

void FPCGToolsetExecutionSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
		World.Reset(GEditor ? GEditor->GetEditorWorldContext(/*bEnsureIsGWorld=*/true).World() : nullptr);
		if (!World)
		{
			return;
		}

		// Destroy() is async, so leftovers from prior runs can still be in the world here.
		for (TActorIterator<APCGVolume> It(World.Get()); It; ++It)
		{
			It->Destroy();
		}

		Graph.Reset(NewObject<UPCGGraph>(GetTransientPackage()));

		/* CreatePointsGrid -> AttributeNoise -> Output. Extents are half-widths
		 * (PointCount = 2*GridExtents/CellSize per axis), so this yields 2x2x1 = 4 points.
		 * AttributeNoise preserves point count, so we can pin that invariant downstream.
		 */
		UPCGSettings* GridSettingsObj = nullptr;
		GridNode.Reset(Graph->AddNodeOfType(UPCGCreatePointsGridSettings::StaticClass(), GridSettingsObj));
		if (UPCGCreatePointsGridSettings* GridSettings = Cast<UPCGCreatePointsGridSettings>(GridSettingsObj))
		{
			GridSettings->GridExtents = FVector(100.0, 100.0, 50.0);
			GridSettings->CellSize = FVector(100.0, 100.0, 100.0);
			GridSettings->bCullPointsOutsideVolume = false;
		}

		UPCGSettings* NoiseSettingsObj = nullptr;
		NoiseNode.Reset(Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), NoiseSettingsObj));

		Graph->AddLabeledEdge(GridNode.Get(), PCGPinConstants::DefaultOutputLabel,
			NoiseNode.Get(), PCGPinConstants::DefaultInputLabel);
		/* OutputNode mirrors the graph's external interface — its input pin uses
		 * DefaultOutputLabel, not DefaultInputLabel.
		 */
		Graph->AddLabeledEdge(NoiseNode.Get(), PCGPinConstants::DefaultOutputLabel,
			Graph->GetOutputNode(), PCGPinConstants::DefaultOutputLabel);

		Volume.Reset(World->SpawnActor<APCGVolume>());
		if (Volume && Volume->PCGComponent)
		{
			Volume->PCGComponent->SetGraphLocal(Graph.Get());
		}
	});

	AfterEach([this]()
	{
		if (Volume)
		{
			Volume->Destroy();
			Volume.Reset();
		}
		GridNode.Reset();
		NoiseNode.Reset();
		Graph.Reset();
		World.Reset();
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("ExecuteGraphInstance"), [this]()
	{
		It(TEXT("runs the volume's graph to completion"), [this]()
		{
			if (!TestNotNull(TEXT("World"), World.Get()) || !TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}

			UPCGExecuteGraphInstanceAsyncResult* Result = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Result = UPCGToolset::ExecuteGraphInstance(Volume.Get());
			});
			if (!TestNotNull(TEXT("AsyncResult"), Result) || !Result)
			{
				return;
			}

			TestTrue(TEXT("Async completed within timeout"), ExecutionTestHelpers::WaitForAsyncResult(Result, World.Get()));
			TestTrue(TEXT("Result marked complete"), Result->bIsComplete);
			TestEqual(TEXT("Result error empty"), Result->Error, FString());
		});

		It(TEXT("populates Value with captured node-level warnings"), [this]()
		{
			if (!TestNotNull(TEXT("World"), World.Get()) ||
				!TestNotNull(TEXT("Volume"), Volume.Get()) ||
				!TestNotNull(TEXT("GridNode"), GridNode.Get()))
			{
				return;
			}

			/* Force the "CellSize must not be less than 0" branch — logs a Warning and returns
			 * success, so the captured message lands in Result->Value.
			 */
			if (UPCGCreatePointsGridSettings* GridSettings =
				Cast<UPCGCreatePointsGridSettings>(GridNode->GetSettings()))
			{
				GridSettings->CellSize = FVector(-1.0, -1.0, -1.0);
			}

			/* We expect a LogPCG warning from CreatePointsGrid, and AddExpectedError alone won't
			 * suppress it cleanly: UE_LOG(Warning) routes through GWarn (the automation message
			 * filter), which downgrades matching messages to Verbose before forwarding them to
			 * GLog — and PCG's capture device drops anything above Warning, leaving Result->Value
			 * empty. So we detach GWarn for the PCG execution so the warning reaches GLog at
			 * original verbosity (capture works) while the test-event suppression at AddEvent
			 * still consults AddExpectedError and keeps the runner output clean.
			 */
			FFeedbackContext* SavedGWarn = GWarn;
			GWarn = nullptr;
			UPCGExecuteGraphInstanceAsyncResult* Result = UPCGToolset::ExecuteGraphInstance(Volume.Get());
			const bool bCompleted = (Result != nullptr) && ExecutionTestHelpers::WaitForAsyncResult(Result, World.Get());
			GWarn = SavedGWarn;

			if (!TestNotNull(TEXT("AsyncResult"), Result) || !Result)
			{
				return;
			}
			if (!TestTrue(TEXT("Async completed"), bCompleted))
			{
				return;
			}

			TestEqual(TEXT("Result error empty (warnings, not failure)"), Result->Error, FString());
			TestTrue(TEXT("At least one node-level message captured"), Result->Value.Num() > 0);

			const bool bHasWarning = Result->Value.ContainsByPredicate(
				[](const FPCGNodeExecutionMessage& Msg) { return Msg.Severity == TEXT("Warning"); });
			TestTrue(TEXT("Captured at least one Warning entry"), bHasWarning);
		});

		It(TEXT("returns a failed AsyncResult when called with a null volume"), [this]()
		{
			// BeforeEach's spawned volume is unused here — we deliberately probe the null path.
			UPCGExecuteGraphInstanceAsyncResult* Result = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Result = UPCGToolset::ExecuteGraphInstance(nullptr);
			});
			if (!TestNotNull(TEXT("AsyncResult"), Result) || !Result)
			{
				return;
			}

			/* SetError runs synchronously on the game thread, so completion + Error are populated
			 * before ExecuteGraphInstance returns. No wait needed.
			 */
			TestTrue(TEXT("Result marked complete"), Result->bIsComplete);
			TestFalse(TEXT("Result error populated"), Result->Error.IsEmpty());
			TestTrue(TEXT("Error message mentions PCGVolume"),
				Result->Error.Contains(TEXT("PCGVolume")));
		});
	});

	Describe(TEXT("GetNodeDataView"), [this]()
	{
		It(TEXT("reflects the upstream point count after execution"), [this]()
		{
			if (!TestNotNull(TEXT("World"), World.Get()) ||
				!TestNotNull(TEXT("Volume"), Volume.Get()) ||
				!TestNotNull(TEXT("NoiseNode"), NoiseNode.Get()))
			{
				return;
			}

			/* GetNodeDataView auto-enables inspection on its first call, so an execute-before-
			 * inspection run captures no per-node data. Enable up-front, then a single
			 * execute-then-query is enough.
			 */
			Volume->PCGComponent->GetExecutionState().GetInspection().EnableInspection();

			UPCGExecuteGraphInstanceAsyncResult* Result = UPCGToolset::ExecuteGraphInstance(Volume.Get());
			if (!TestNotNull(TEXT("AsyncResult"), Result))
			{
				return;
			}
			if (!TestTrue(TEXT("Async completed"), ExecutionTestHelpers::WaitForAsyncResult(Result, World.Get())))
			{
				return;
			}

			FString DataView;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				DataView = UPCGToolset::GetNodeDataView(Volume.Get(), NoiseNode.Get(), TEXT("Out"),
					/*AttributeName=*/TEXT(""), /*StartIndex=*/0, /*EndIndex=*/-1);
			});

			TestFalse(TEXT("DataView non-empty"), DataView.IsEmpty());
			TestEqual(TEXT("DataView contains exactly 4 element entries"),
				ExecutionTestHelpers::CountElementKeys(DataView), 4);
		});

		It(TEXT("errors when no execution data is available"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()) || !TestNotNull(TEXT("NoiseNode"), NoiseNode.Get()))
			{
				return;
			}
			AddExpectedErrorPlain(TEXT("No execution data"));

			/* No execution before this call. Pin the error path so a future refactor that
			 * silently swallows it is caught.
			 */
			FString DataView;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				DataView = UPCGToolset::GetNodeDataView(Volume.Get(), NoiseNode.Get(), TEXT("Out"));
			});

			TestTrue(TEXT("DataView empty on error"), DataView.IsEmpty());
			ExpectExceptionContains(TEXT("No execution data"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
