// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Misc/AutomationTest.h"
#include "PCGComponent.h"
#include "PCGGraph.h"
#include "PCGToolset.h"
#include "PCGVolume.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/ValueOrError.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace GraphInstanceTestHelpers
{
	void AddFloatUserParam(UPCGGraph* Graph, FName Name)
	{
		FPropertyBagPropertyDesc Desc(Name, EPropertyBagPropertyType::Float);
		Graph->AddUserParameters({Desc});
	}

	const FProperty* FindParamProperty(UPCGGraphInstance* Instance, FName Name)
	{
		const FInstancedPropertyBag& Bag = Instance->ParametersOverrides.Parameters;
		const FPropertyBagPropertyDesc* Desc = Bag.FindPropertyDescByName(Name);
		return Desc ? Desc->CachedProperty : nullptr;
	}
}

BEGIN_DEFINE_SPEC(FPCGToolsetGraphInstanceSpec, "AI.Toolsets.PCGToolset.GraphInstance",
	PCGToolsetTest::Flags)
	PCG_TEST_EXCEPTION_HELPERS()

	TStrongObjectPtr<UWorld> World;
	TStrongObjectPtr<UPCGGraph> Graph;
	TStrongObjectPtr<APCGVolume> Volume;
	TStrongObjectPtr<APCGVolume> SpawnedVolume;  // tracked by SpawnGraphInstance test for cleanup
END_DEFINE_SPEC(FPCGToolsetGraphInstanceSpec)

void FPCGToolsetGraphInstanceSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
		World.Reset(GEditor ? GEditor->GetEditorWorldContext(/*bEnsureIsGWorld=*/true).World() : nullptr);
		if (!World)
		{
			return;
		}
		/* Editor-world Destroy() is async, so leftovers from a prior run can still be
		 * discoverable here and break ListGraphInstances assertions or collide with names.
		 */
		for (TActorIterator<APCGVolume> It(World.Get()); It; ++It)
		{
			It->Destroy();
		}
		Graph.Reset(NewObject<UPCGGraph>(GetTransientPackage()));
		Volume.Reset(World->SpawnActor<APCGVolume>());
		if (Volume && Volume->PCGComponent)
		{
			Volume->PCGComponent->SetGraphLocal(Graph.Get());
		}
	});

	AfterEach([this]()
	{
		if (SpawnedVolume)
		{
			SpawnedVolume->Destroy();
			SpawnedVolume.Reset();
		}
		if (Volume)
		{
			Volume->Destroy();
			Volume.Reset();
		}
		Graph.Reset();
		World.Reset();
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("ListGraphInstances"), [this]()
	{
		It(TEXT("discovers the spawned APCGVolume"), [this]()
		{
			if (!TestNotNull(TEXT("World"), World.Get()) || !TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}

			TArray<FPCGGraphInstanceInfo> Instances;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Instances = UPCGToolset::ListGraphInstances();
			});

			const FPCGGraphInstanceInfo* Found = Instances.FindByPredicate(
				[this](const FPCGGraphInstanceInfo& Info) { return Info.Actor == Volume.Get(); });
			if (TestNotNull(TEXT("Volume listed"), Found))
			{
				TestEqual(TEXT("Reported graph"), Found->Graph.Get(), Graph.Get());
			}
		});
	});

	Describe(TEXT("SpawnGraphInstance"), [this]()
	{
		It(TEXT("spawns a PCGVolume backed by the requested graph"), [this]()
		{
			if (!TestNotNull(TEXT("World"), World.Get()))
			{
				return;
			}

			FPCGGraphInstanceInfo Info;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Info = UPCGToolset::SpawnGraphInstance(
					Graph.Get(), TEXT("AutoSpawnVol"), FTransform::Identity, FString());
			});
			SpawnedVolume.Reset(Info.Actor);  // claim for cleanup in AfterEach

			if (!TestNotNull(TEXT("Spawned actor"), SpawnedVolume.Get()))
			{
				return;
			}
			TestEqual(TEXT("Reported graph"), Info.Graph.Get(), Graph.Get());

			// Discoverable via the world iterator that ListGraphInstances itself uses.
			bool bFoundInWorld = false;
			for (TActorIterator<APCGVolume> It(World.Get()); It; ++It)
			{
				if (*It == SpawnedVolume.Get())
				{
					bFoundInWorld = true;
					break;
				}
			}
			TestTrue(TEXT("Spawned actor in editor world"), bFoundInWorld);
		});
	});

	Describe(TEXT("GetGraphInstanceParams"), [this]()
	{
		It(TEXT("returns the override bag with the graph's user parameters"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());  // refresh instance bag

			FInstancedPropertyBag Result;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Result = UPCGToolset::GetGraphInstanceParams(Volume.Get());
			});
			TestNotNull(TEXT("Density present"),
				Result.FindPropertyDescByName(FName(TEXT("Density"))));
		});
	});

	Describe(TEXT("SetGraphInstanceParams"), [this]()
	{
		It(TEXT("writes JSON overrides into the graph instance bag"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetGraphInstanceParams(Volume.Get(), TEXT("{\"Density\": 0.42}"));
			});
			TestTrue(TEXT("SetGraphInstanceParams succeeded"), bSuccess);

			UPCGGraphInstance* Instance = Volume->PCGComponent->GetGraphInstance();
			if (!TestNotNull(TEXT("Graph instance"), Instance))
			{
				return;
			}
			const FProperty* DensityProp = GraphInstanceTestHelpers::FindParamProperty(Instance, FName(TEXT("Density")));
			if (TestNotNull(TEXT("Density property reflected"), DensityProp))
			{
				TestTrue(TEXT("Density marked overridden"),
					Instance->IsPropertyOverridden(DensityProp));
			}

			const TValueOrError<float, EPropertyBagResult> Value =
				Instance->ParametersOverrides.Parameters.GetValueFloat(FName(TEXT("Density")));
			if (TestTrue(TEXT("Density read back ok"), Value.IsValid()))
			{
				TestEqual(TEXT("Density value"), Value.GetValue(), 0.42f);
			}
		});

		It(TEXT("fails quietly on malformed JSON"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());

			/* SetGraphInstanceParams returns false on bad JSON without raising. Pin this
			 * fail-safe-quiet contract so a future unguarded raise would surface here.
			 */
			bool bSuccess = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetGraphInstanceParams(Volume.Get(), TEXT("{broken: json"));
			});
			TestFalse(TEXT("SetGraphInstanceParams returns false on bad JSON"), bSuccess);
			ExpectNoException();

			UPCGGraphInstance* Instance = Volume->PCGComponent->GetGraphInstance();
			if (TestNotNull(TEXT("Graph instance"), Instance))
			{
				const FProperty* DensityProp = GraphInstanceTestHelpers::FindParamProperty(Instance, FName(TEXT("Density")));
				if (TestNotNull(TEXT("Density property reflected"), DensityProp))
				{
					TestFalse(TEXT("Density not marked overridden after failed parse"),
						Instance->IsPropertyOverridden(DensityProp));
				}
			}
		});

		It(TEXT("ignores valid JSON that names a non-existent param"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());

			/* Unknown keys are silently skipped, and the property bag's deserializer tolerates
			 * unknown fields. Pin the invariant the AI agent relies on: an unknown param name
			 * does NOT spuriously mark a real param as overridden, and the call still returns true.
			 */
			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetGraphInstanceParams(Volume.Get(), TEXT("{\"Ghost\": 99}"));
			});
			TestTrue(TEXT("SetGraphInstanceParams returns true on tolerated unknown key"), bSuccess);

			UPCGGraphInstance* Instance = Volume->PCGComponent->GetGraphInstance();
			if (!TestNotNull(TEXT("Graph instance"), Instance))
			{
				return;
			}
			const FProperty* DensityProp = GraphInstanceTestHelpers::FindParamProperty(Instance, FName(TEXT("Density")));
			if (TestNotNull(TEXT("Density property reflected"), DensityProp))
			{
				TestFalse(TEXT("Density not marked overridden by unknown-key write"),
					Instance->IsPropertyOverridden(DensityProp));
			}
			TestNull(TEXT("Ghost did not get added to the bag"),
				Instance->ParametersOverrides.Parameters.FindPropertyDescByName(FName(TEXT("Ghost"))));
		});
	});

	Describe(TEXT("ResetGraphInstanceParams"), [this]()
	{
		It(TEXT("clears a previously-set override"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());

			ExceptionHandler->CaptureErrorsIn([&]()
			{
				UPCGToolset::SetGraphInstanceParams(Volume.Get(), TEXT("{\"Density\": 0.42}"));
			});

			bool bReset = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bReset = UPCGToolset::ResetGraphInstanceParams(Volume.Get(), {FName(TEXT("Density"))});
			});
			TestTrue(TEXT("ResetGraphInstanceParams succeeded"), bReset);

			UPCGGraphInstance* Instance = Volume->PCGComponent->GetGraphInstance();
			if (!TestNotNull(TEXT("Graph instance"), Instance))
			{
				return;
			}
			const FProperty* DensityProp = GraphInstanceTestHelpers::FindParamProperty(Instance, FName(TEXT("Density")));
			if (TestNotNull(TEXT("Density property reflected"), DensityProp))
			{
				TestFalse(TEXT("Density no longer overridden"),
					Instance->IsPropertyOverridden(DensityProp));
			}
		});

		It(TEXT("reports param names that don't exist on the instance"), [this]()
		{
			if (!TestNotNull(TEXT("Volume"), Volume.Get()))
			{
				return;
			}
			AddExpectedErrorPlain(TEXT("the following parameters were not found"));
			GraphInstanceTestHelpers::AddFloatUserParam(Graph.Get(), FName(TEXT("Density")));
			Volume->PCGComponent->SetGraphLocal(Graph.Get());

			bool bReset = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bReset = UPCGToolset::ResetGraphInstanceParams(Volume.Get(),
					{FName(TEXT("Density")), FName(TEXT("Ghost"))});
			});
			TestFalse(TEXT("ResetGraphInstanceParams rejects mixed valid/invalid"), bReset);
			ExpectExceptionContains(TEXT("Ghost"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
