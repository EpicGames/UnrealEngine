// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/AutomationTest.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGToolset.h"
#include "Elements/PCGAttributeNoise.h"
#include "Metadata/PCGMetadataCommon.h"
#include "StructUtils/PropertyBag.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/SoftObjectPath.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FPCGToolsetGraphSpec, "AI.Toolsets.PCGToolset.Graph", PCGToolsetTest::Flags)
	PCG_TEST_EXCEPTION_HELPERS()
END_DEFINE_SPEC(FPCGToolsetGraphSpec)

void FPCGToolsetGraphSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
	});

	AfterEach([this]()
	{
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("CreateGraph"), [this]()
	{
		It(TEXT("creates a saved asset at the requested path"), [this]()
		{
			PCGToolsetTest::FAssetSandbox Sandbox;
			UPCGGraph* Graph = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Graph = UPCGToolset::CreateGraph(
					TEXT("AutoTestCreate"), PCGToolsetTest::FAssetSandbox::GetDefaultPath());
			});
			if (TestNotNull(TEXT("Graph created"), Graph))
			{
				Sandbox.Track(Graph);
				TestEqual(TEXT("Asset name"), Graph->GetName(), TEXT("AutoTestCreate"));
			}
		});
	});

	Describe(TEXT("GetGraphStructure"), [this]()
	{
		It(TEXT("reflects nodes and edges added through the PCG graph API"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph(TEXT("StructureGraph"));
			Graph->Description = FText::FromString(TEXT("test desc"));

			UPCGSettings* IgnoredA = nullptr;
			UPCGSettings* IgnoredB = nullptr;
			UPCGNode* NodeA = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredA);
			UPCGNode* NodeB = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredB);
			if (!TestNotNull(TEXT("NodeA added"), NodeA) || !TestNotNull(TEXT("NodeB added"), NodeB))
			{
				return;
			}
			NodeA->Rename(TEXT("NodeA"));
			NodeB->Rename(TEXT("NodeB"));

			const TArray<FPCGPinProperties> Outputs = NodeA->OutputPinProperties();
			const TArray<FPCGPinProperties> Inputs = NodeB->InputPinProperties();
			if (!TestTrue(TEXT("NodeA has output pins"), Outputs.Num() > 0) ||
				!TestTrue(TEXT("NodeB has input pins"), Inputs.Num() > 0))
			{
				return;
			}
			const FName OutLabel = Outputs[0].Label;
			const FName InLabel = Inputs[0].Label;
			/* AddLabeledEdge's bool return reports "broke incompatible edges", not success —
			 * verify by inspecting the destination pin instead.
			 */
			Graph->AddLabeledEdge(NodeA, OutLabel, NodeB, InLabel);
			const UPCGPin* DestPin = NodeB->GetInputPin(InLabel);
			if (!TestTrue(TEXT("Destination pin connected"), DestPin && DestPin->IsConnected()))
			{
				return;
			}

			FPCGGraphStructure Structure;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Structure = UPCGToolset::GetGraphStructure(Graph);
			});

			TestEqual(TEXT("Name"), Structure.Name, TEXT("StructureGraph"));
			TestEqual(TEXT("Description"), Structure.Description, TEXT("test desc"));

			// Two user nodes plus the synthetic Input/Output nodes.
			TestEqual(TEXT("Nodes count"), Structure.Nodes.Num(), 4);
			const auto HasNode = [&](const FString& Name)
			{
				return Structure.Nodes.ContainsByPredicate(
					[&](const FPCGNodeInfo& Info) { return Info.Name == Name; });
			};
			TestTrue(TEXT("NodeA reported"), HasNode(TEXT("NodeA")));
			TestTrue(TEXT("NodeB reported"), HasNode(TEXT("NodeB")));

			TestEqual(TEXT("Edge count"), Structure.Edges.Num(), 1);
			if (Structure.Edges.Num() == 1)
			{
				const FPCGEdgeInfo& Edge = Structure.Edges[0];
				TestEqual(TEXT("Edge.SrcNode"), Edge.SrcNode, TEXT("NodeA"));
				TestEqual(TEXT("Edge.DestNode"), Edge.DestNode, TEXT("NodeB"));
				TestEqual(TEXT("Edge.SrcPin"), Edge.SrcPin, OutLabel.ToString());
				TestEqual(TEXT("Edge.DestPin"), Edge.DestPin, InLabel.ToString());
			}
		});
	});

	Describe(TEXT("SetGraphParams"), [this]()
	{
		It(TEXT("adds a parameter to the graph's user-parameters bag"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph();

			FPCGParamDefinition Density;
			Density.Name = TEXT("Density");
			Density.Type = EPCGMetadataTypes::Float;
			Density.Description = TEXT("density factor");

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetGraphParams(Graph, {Density});
			});
			TestTrue(TEXT("SetGraphParams succeeded"), bSuccess);

			const FInstancedPropertyBag* Bag = Graph->GetUserParametersStruct();
			if (TestNotNull(TEXT("Graph has a user-parameters bag"), Bag))
			{
				TestNotNull(TEXT("Density param exists in bag"),
					Bag->FindPropertyDescByName(FName(TEXT("Density"))));
			}
		});

	});

	Describe(TEXT("RemoveGraphParams"), [this]()
	{
		It(TEXT("removes a parameter from the graph's user-parameters bag"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph();

			FPCGParamDefinition Density;
			Density.Name = TEXT("Density");
			Density.Type = EPCGMetadataTypes::Float;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				UPCGToolset::SetGraphParams(Graph, {Density});
			});

			bool bRemoved = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bRemoved = UPCGToolset::RemoveGraphParams(Graph, {FName(TEXT("Density"))});
			});
			TestTrue(TEXT("RemoveGraphParams succeeded"), bRemoved);

			const FInstancedPropertyBag* Bag = Graph->GetUserParametersStruct();
			if (TestNotNull(TEXT("Graph has a user-parameters bag"), Bag))
			{
				TestNull(TEXT("Density param removed from bag"),
					Bag->FindPropertyDescByName(FName(TEXT("Density"))));
			}
		});

	});

	Describe(TEXT("GetGraphSchema"), [this]()
	{
		It(TEXT("returns the graph name"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph(TEXT("SchemaGraph"));

			FPCGGraphSchema Schema;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Schema = UPCGToolset::GetGraphSchema(Graph);
			});
			TestEqual(TEXT("Name"), Schema.Name, TEXT("SchemaGraph"));
		});
	});

	Describe(TEXT("GetGraphDescription"), [this]()
	{
		It(TEXT("returns the graph's description"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph();
			Graph->Description = FText::FromString(TEXT("hello-desc"));

			FString Description;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Description = UPCGToolset::GetGraphDescription(Graph);
			});
			TestEqual(TEXT("Description"), Description, TEXT("hello-desc"));
		});
	});

	Describe(TEXT("SetGraphDescription"), [this]()
	{
		It(TEXT("updates the graph description"), [this]()
		{
			UPCGGraph* Graph = PCGToolsetTest::MakeTransientGraph();

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetGraphDescription(Graph, TEXT("hello"));
			});
			TestTrue(TEXT("SetGraphDescription succeeded"), bSuccess);
			TestEqual(TEXT("Description applied"), Graph->Description.ToString(), TEXT("hello"));
		});
	});

	Describe(TEXT("ListAvailableSubgraphs"), [this]()
	{
		It(TEXT("returns paths that resolve to UPCGGraph assets"), [this]()
		{
			TArray<FString> Subgraphs;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Subgraphs = UPCGToolset::ListAvailableSubgraphs();
			});

			/* In a bare test environment the list is often empty; the loop earns its keep
			 * when subgraphs are configured (CI with content, project-level test).
			 */
			const IAssetRegistry& Registry =
				FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			const FTopLevelAssetPath GraphClass = UPCGGraph::StaticClass()->GetClassPathName();
			for (const FString& Path : Subgraphs)
			{
				const FAssetData Data = Registry.GetAssetByObjectPath(FSoftObjectPath(Path));
				TestEqual(*FString::Printf(TEXT("'%s' asset class"), *Path),
					Data.AssetClassPath, GraphClass);
			}
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
