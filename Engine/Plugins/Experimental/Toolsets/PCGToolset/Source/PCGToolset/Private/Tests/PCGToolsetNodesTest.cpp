// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "PCGGraph.h"
#include "PCGNode.h"
#include "PCGPin.h"
#include "PCGSettings.h"
#include "PCGToolset.h"
#include "PCGToolsetLibraryCore.h"
#include "Elements/PCGAttributeNoise.h"
#include "Tests/PCGToolsetTestFixture.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace NodesTestHelpers
{
	// Node type used for tests that need a stable AICallable node and pin shape.
	const TCHAR* const TestNodeType = TEXT("Attribute Noise");
}

BEGIN_DEFINE_SPEC(FPCGToolsetNodesSpec, "AI.Toolsets.PCGToolset.Nodes", PCGToolsetTest::Flags)
	PCG_TEST_EXCEPTION_HELPERS()

	TStrongObjectPtr<UPCGGraph> Graph;
END_DEFINE_SPEC(FPCGToolsetNodesSpec)

void FPCGToolsetNodesSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
		Graph.Reset(PCGToolsetTest::MakeTransientGraph());
	});

	AfterEach([this]()
	{
		Graph.Reset();
		if (ExceptionHandler.IsValid())
		{
			ExpectNoException();
		}
	});

	Describe(TEXT("AddNode"), [this]()
	{
		It(TEXT("adds a node of the requested native type and applies JsonParams"), [this]()
		{
			UPCGNode* Node = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Node = UPCGToolset::AddNode(Graph.Get(), NodesTestHelpers::TestNodeType, TEXT("N1"),
					TEXT("{\"NoiseMin\": 0.25, \"NoiseMax\": 0.75}"));
			});
			if (!TestNotNull(TEXT("Node added"), Node))
			{
				return;
			}
			TestTrue(TEXT("Graph contains node"), Graph->GetNodes().Contains(Node));

			const UPCGAttributeNoiseSettings* Settings =
				Cast<UPCGAttributeNoiseSettings>(Node->GetSettings());
			if (TestNotNull(TEXT("Attribute Noise settings"), Settings))
			{
				TestEqual(TEXT("NoiseMin applied"), Settings->NoiseMin, 0.25f);
				TestEqual(TEXT("NoiseMax applied"), Settings->NoiseMax, 0.75f);
			}
		});

		It(TEXT("rejects an unknown native type"), [this]()
		{
			AddExpectedErrorPlain(TEXT("does not exist"));
			UPCGNode* Node = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Node = UPCGToolset::AddNode(Graph.Get(), TEXT("NotARealNode"), TEXT("N1"));
			});
			TestNull(TEXT("Unknown type yields no node"), Node);
			ExpectExceptionContains(TEXT("does not exist"));
		});
	});

	Describe(TEXT("AddSubgraphNode"), [this]()
	{
		It(TEXT("wraps another UPCGGraph as a subgraph node"), [this]()
		{
			UPCGGraph* Sub = PCGToolsetTest::MakeTransientGraph(TEXT("SubGraph"));
			UPCGNode* Node = nullptr;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Node = UPCGToolset::AddSubgraphNode(Graph.Get(), Sub, TEXT("S1"));
			});
			if (TestNotNull(TEXT("Subgraph node added"), Node))
			{
				TestTrue(TEXT("Graph contains subgraph node"), Graph->GetNodes().Contains(Node));
			}
		});
	});

	Describe(TEXT("UpdateNode"), [this]()
	{
		It(TEXT("updates the node title and applies JsonParams"), [this]()
		{
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::UpdateNode(Node,
					TEXT("{\"NoiseMin\": 0.1, \"bClampResult\": true}"), TEXT("New Title"));
			});
			TestTrue(TEXT("UpdateNode succeeded"), bSuccess);
			TestEqual(TEXT("Title applied"), Node->NodeTitle, FName(TEXT("New Title")));

			const UPCGAttributeNoiseSettings* Settings =
				Cast<UPCGAttributeNoiseSettings>(Node->GetSettings());
			if (TestNotNull(TEXT("Attribute Noise settings"), Settings))
			{
				TestEqual(TEXT("NoiseMin applied"), Settings->NoiseMin, 0.1f);
				TestTrue(TEXT("bClampResult applied"), Settings->bClampResult);
			}
		});

		It(TEXT("rejects malformed JsonParams"), [this]()
		{
			AddExpectedErrorPlain(TEXT("Could not set params on node"));
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bSuccess = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::UpdateNode(Node, TEXT("{not json"), TEXT(""));
			});
			TestFalse(TEXT("UpdateNode rejects malformed JSON"), bSuccess);
			ExpectException();
		});

		It(TEXT("rejects valid JSON that names a non-existent setting"), [this]()
		{
			/* Two errors fire: SetObjectProperties raises naming the unknown key, then
			 * UpdateNode wraps with its own "Could not set params on node" message.
			 */
			AddExpectedErrorPlain(TEXT("could not be set"));
			AddExpectedErrorPlain(TEXT("Could not set params on node"));
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bSuccess = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::UpdateNode(Node,
					TEXT("{\"NotARealSetting\": 42}"), TEXT(""));
			});
			TestFalse(TEXT("UpdateNode rejects unknown setting name"), bSuccess);
			ExpectExceptionContains(TEXT("NotARealSetting"));
		});
	});

	Describe(TEXT("SetNodeComment"), [this]()
	{
		It(TEXT("sets the comment text on the node"), [this]()
		{
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::SetNodeComment(Node, TEXT("hi"));
			});
			TestTrue(TEXT("SetNodeComment succeeded"), bSuccess);
			TestEqual(TEXT("Comment applied"), Node->NodeComment, TEXT("hi"));
		});
	});

	Describe(TEXT("GetNodeInfo"), [this]()
	{
		It(TEXT("returns the node's name"), [this]()
		{
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}
			Node->Rename(TEXT("N1"));

			FPCGNodeInfo Info;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Info = UPCGToolset::GetNodeInfo(Node);
			});
			TestEqual(TEXT("Node name"), Info.Name, TEXT("N1"));
		});
	});

	Describe(TEXT("RepositionNode"), [this]()
	{
		It(TEXT("updates the node position"), [this]()
		{
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bSuccess = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bSuccess = UPCGToolset::RepositionNode(Node, 7, 11);
			});
			TestTrue(TEXT("RepositionNode succeeded"), bSuccess);

			int32 X = 0, Y = 0;
			Node->GetNodePosition(X, Y);
			TestEqual(TEXT("Position X"), X, 7);
			TestEqual(TEXT("Position Y"), Y, 11);
		});
	});

	Describe(TEXT("RemoveNode"), [this]()
	{
		It(TEXT("removes the node from the graph"), [this]()
		{
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Node = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Seed node"), Node))
			{
				return;
			}

			bool bRemoved = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bRemoved = UPCGToolset::RemoveNode(Graph.Get(), Node);
			});
			TestTrue(TEXT("RemoveNode succeeded"), bRemoved);
			TestFalse(TEXT("Graph no longer contains the node"), Graph->GetNodes().Contains(Node));
		});

		It(TEXT("rejects a node from a different graph"), [this]()
		{
			AddExpectedErrorPlain(TEXT("should belong to the Graph"));
			UPCGGraph* OtherGraph = PCGToolsetTest::MakeTransientGraph();
			UPCGSettings* IgnoredDefaults = nullptr;
			UPCGNode* Foreign = OtherGraph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredDefaults);
			if (!TestNotNull(TEXT("Foreign node added"), Foreign))
			{
				return;
			}

			bool bRemoved = true;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bRemoved = UPCGToolset::RemoveNode(Graph.Get(), Foreign);
			});
			TestFalse(TEXT("RemoveNode rejects cross-graph node"), bRemoved);
			ExpectExceptionContains(TEXT("should belong to the Graph"));
			TestTrue(TEXT("Foreign node still on its own graph"),
				OtherGraph->GetNodes().Contains(Foreign));
		});
	});

	Describe(TEXT("ConnectNodePins / DisconnectNodePins"), [this]()
	{
		It(TEXT("connects then disconnects a self-compatible node pair"), [this]()
		{
			UPCGSettings* IgnoredA = nullptr;
			UPCGSettings* IgnoredB = nullptr;
			UPCGNode* From = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredA);
			UPCGNode* To = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredB);
			if (!TestNotNull(TEXT("From added"), From) || !TestNotNull(TEXT("To added"), To))
			{
				return;
			}

			TArray<FPCGPinProperties> Outputs = From->OutputPinProperties();
			TArray<FPCGPinProperties> Inputs = To->InputPinProperties();
			if (!TestTrue(TEXT("From has output pins"), Outputs.Num() > 0) ||
				!TestTrue(TEXT("To has input pins"), Inputs.Num() > 0))
			{
				return;
			}
			const FString OutLabel = Outputs[0].Label.ToString();
			const FString InLabel = Inputs[0].Label.ToString();

			ExceptionHandler->CaptureErrorsIn([&]()
			{
				UPCGToolset::ConnectNodePins(From, OutLabel, To, InLabel);
			});

			const UPCGPin* DestPin = To->GetInputPin(Inputs[0].Label);
			TestTrue(TEXT("Destination pin reports connected after Connect"),
				DestPin && DestPin->IsConnected());

			bool bDisconnected = false;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				bDisconnected = UPCGToolset::DisconnectNodePins(From, OutLabel, To, InLabel);
			});
			TestTrue(TEXT("DisconnectNodePins succeeded"), bDisconnected);
			TestFalse(TEXT("Destination pin no longer connected after Disconnect"),
				DestPin && DestPin->IsConnected());
		});

		It(TEXT("rejects connecting nodes that live on different graphs"), [this]()
		{
			AddExpectedErrorPlain(TEXT("should belong to the same graph"));
			UPCGGraph* OtherGraph = PCGToolsetTest::MakeTransientGraph();
			UPCGSettings* IgnoredA = nullptr;
			UPCGSettings* IgnoredB = nullptr;
			UPCGNode* From = Graph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredA);
			UPCGNode* To = OtherGraph->AddNodeOfType(UPCGAttributeNoiseSettings::StaticClass(), IgnoredB);
			if (!TestNotNull(TEXT("From added"), From) || !TestNotNull(TEXT("To added"), To))
			{
				return;
			}

			TArray<UPCGNode*> Added;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Added = UPCGToolset::ConnectNodePins(From,
					From->OutputPinProperties()[0].Label.ToString(), To,
					To->InputPinProperties()[0].Label.ToString());
			});
			TestEqual(TEXT("ConnectNodePins returns no nodes"), Added.Num(), 0);
			ExpectExceptionContains(TEXT("should belong to the same graph"));
		});
	});

	Describe(TEXT("ListNativeNodes"), [this]()
	{
		It(TEXT("returns names that resolve to non-null PCG settings classes"), [this]()
		{
			TArray<FString> Nodes;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Nodes = UPCGToolset::ListNativeNodes(/*bCommonOnly=*/true);
			});

			const TMap<FName, UPCGSettings*>& Map =
				PCGToolsetLibrary::Graph::GetNodeNameToSettingsMap();
			for (const FString& Name : Nodes)
			{
				UPCGSettings* const* Found = Map.Find(FName(*Name));
				if (TestNotNull(*FString::Printf(TEXT("'%s' present in node map"), *Name), Found))
				{
					TestNotNull(*FString::Printf(TEXT("'%s' settings non-null"), *Name), *Found);
				}
			}
		});

		It(TEXT("with bCommonOnly=false returns the entire registry"), [this]()
		{
			TArray<FString> Common, All;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Common = UPCGToolset::ListNativeNodes(/*bCommonOnly=*/true);
				All = UPCGToolset::ListNativeNodes(/*bCommonOnly=*/false);
			});

			const TMap<FName, UPCGSettings*>& Map =
				PCGToolsetLibrary::Graph::GetNodeNameToSettingsMap();
			TestEqual(TEXT("All-list size matches registry size"), All.Num(), Map.Num());
			TestTrue(TEXT("All-list strictly larger than common-list"), All.Num() > Common.Num());
			for (const FString& Name : Common)
			{
				TestTrue(*FString::Printf(TEXT("Common node '%s' also in full list"), *Name),
					All.Contains(Name));
			}
		});
	});

	Describe(TEXT("GetNativeNodeSchema"), [this]()
	{
		It(TEXT("returns the requested node's name"), [this]()
		{
			FPCGNativeNodeSchema Schema;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Schema = UPCGToolset::GetNativeNodeSchema(NodesTestHelpers::TestNodeType);
			});
			TestEqual(TEXT("Schema name"), Schema.Name, FString(NodesTestHelpers::TestNodeType));
		});

		It(TEXT("rejects an unknown node type"), [this]()
		{
			AddExpectedErrorPlain(TEXT("not in available nodes"));
			FPCGNativeNodeSchema Schema;
			ExceptionHandler->CaptureErrorsIn([&]()
			{
				Schema = UPCGToolset::GetNativeNodeSchema(TEXT("NotARealNode"));
			});
			TestEqual(TEXT("Unknown name yields empty schema"), Schema.Name, FString());
			ExpectExceptionContains(TEXT("not in available nodes"));
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
