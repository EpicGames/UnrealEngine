// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "EditorAssetLibrary.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "PCGGraph.h"
#include "Templates/UniquePtr.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"
#include "UObject/Package.h"

namespace PCGToolsetTest
{
	// Default flags applied to every spec in this plugin.
	const auto Flags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;

	inline UPCGGraph* MakeTransientGraph(const FName Name = NAME_None)
	{
		return NewObject<UPCGGraph>(GetTransientPackage(), Name);
	}

	/* RAII for tests that need a saved UPCGGraph asset path. Registers /Automation/ on the
	 * project's automation-transient dir so CreateAsset resolves to a writable location, and
	 * unregisters + deletes tracked assets on destruction.
	 */
	class FAssetSandbox
	{
	public:
		static const TCHAR* GetMountPoint() { return TEXT("/Automation/"); }
		static const TCHAR* GetDefaultPath() { return TEXT("/Automation/PCG"); }

		FAssetSandbox()
		{
			MountDir = FPaths::AutomationTransientDir() / TEXT("PCG");
			/* Wipe leftovers from a prior run that crashed or whose DeleteAsset was soft-blocked.
			 * Without this, a second run can collide with a stale .uasset and CreateAsset returns
			 * the old object instead of a fresh one.
			 */
			IFileManager::Get().DeleteDirectory(*MountDir, /*RequireExists=*/false, /*Tree=*/true);
			IFileManager::Get().MakeDirectory(*MountDir, /*Tree=*/true);
			FPackageName::RegisterMountPoint(GetMountPoint(), MountDir);
		}

		~FAssetSandbox()
		{
			for (const FString& Path : Created)
			{
				UEditorAssetLibrary::DeleteAsset(Path);
			}
			FPackageName::UnRegisterMountPoint(GetMountPoint(), MountDir);
		}

		FAssetSandbox(const FAssetSandbox&) = delete;
		FAssetSandbox& operator=(const FAssetSandbox&) = delete;

		/* Track an asset for deletion in the destructor. Records the package name (e.g.
		 * "/Automation/PCG/AutoTestCreate") since that's the form DeleteAsset expects.
		 */
		void Track(const UObject* Asset)
		{
			if (Asset && Asset->GetPackage())
			{
				Created.Add(Asset->GetPackage()->GetName());
			}
		}

	private:
		TArray<FString> Created;
		FString MountDir;
	};
}  // namespace PCGToolsetTest

/* Drops the standard ExceptionHandler storage + assertion helpers into a BEGIN_DEFINE_SPEC
 * body. Use CaptureErrorsIn([&]{...}) then ExpectNoException / ExpectException /
 * ExpectExceptionContains to assert what fired.
 */
#define PCG_TEST_EXCEPTION_HELPERS()                                                       \
	void ExpectNoException()                                                               \
	{                                                                                      \
		check(ExceptionHandler.IsValid());                                                 \
		TestEqual(TEXT("Error"), *ExceptionHandler->GetException(), TEXT(""));             \
		ExceptionHandler.Reset();                                                          \
	}                                                                                      \
	void ExpectException()                                                                 \
	{                                                                                      \
		check(ExceptionHandler.IsValid());                                                 \
		TestNotEqual(TEXT("Has Error"), *ExceptionHandler->GetException(), TEXT(""));      \
		ExceptionHandler.Reset();                                                          \
	}                                                                                      \
	void ExpectExceptionContains(const FString& ErrorMessage)                              \
	{                                                                                      \
		check(ExceptionHandler.IsValid());                                                 \
		TestTrue(                                                                          \
			*FString::Printf(TEXT("Error contains '%s'"), *ErrorMessage),                  \
			ExceptionHandler->GetException().Contains(ErrorMessage));                      \
		ExceptionHandler.Reset();                                                          \
	}                                                                                      \
	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;

#endif  // WITH_DEV_AUTOMATION_TESTS
