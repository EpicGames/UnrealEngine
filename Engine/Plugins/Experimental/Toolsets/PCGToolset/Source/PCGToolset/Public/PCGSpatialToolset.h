// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGToolsetCustomTypes.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "PCGSpatialToolset.generated.h"

UCLASS(BlueprintType)
class UPCGSpatialToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	// Graph -----------------------------------------------------------------------------------------------------------
	/**
	 * Runs an instant PCG graph with the specified parameters in fire-and-forget mode
	 * (Should be called directly: Not callable in a python context execution context)
	 *
	 * @param Graph - The PCG graph to execute
	 * @param Params - Key-value parameters to pass to the graph
	 * @return Array of per-node execution messages emitted while running the graph (empty on success with no issues).
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Graph")
	static UPCGExecuteGraphInstanceAsyncResult* RunPCGInstantGraph(UPCGGraph* Graph, const TMap<FString, FString>& Params);
};
