// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGToolsetCustomTypes.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "PCGToolset.generated.h"

class UToolCallAsyncResultVoid;

/**
 *  Toolset for building and modifying PCG graphs
 */
UCLASS(BlueprintType)
class UPCGToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	// Graph -----------------------------------------------------------------------------------------------------------
	/**
	 * Creates a new saved PCG graph asset.
	 *
	 * @param Name - Name for the new PCG graph asset (e.g., "PCG_ForestScatter")
	 * @param Path - Content folder path where the asset should be created. Defaults to "/Game/PCG"
	 * @return The created PCG graph
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Graph")
	static UPCGGraph* CreateGraph(const FString& Name, const FString& Path = TEXT("/Game/PCG"));

	// Graph Params -----------------------------------------------------------------------------------------------------
	/**
	 * Returns the complete structure of a PCG graph including all nodes,
	 * connections, exposed parameters, and comment boxes.
	 *
	 * @param Graph - Graph to inspect
	 * @return FPCGGraphStructure with graph name, description, nodes, and edges
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static FPCGGraphStructure GetGraphStructure(const UPCGGraph* Graph);

	/**
	 * Adds one or more graph user parameters to a specific PCG graph, such that they will be overridable in per graph instance.
	 *
	 * @param Graph - Graph to add graph parameter to
	 * @param Params - TArray<FPCGParamDefinition> An array of UStruct for Name, Type, Description, ContainerType (optional) and DefaultValueJson (optional). If user explicitly want special default values the DefaultValueJson MUST be set, otherwise OMIT DefaultValueJson for standard UE default values!
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static bool SetGraphParams(UPCGGraph* Graph, const TArray<FPCGParamDefinition>& Params);

	/**
	 * Removes graph parameters to a specific PCG graph, such that they are not overridable anymore.
	 *
	 * @param Graph - Graph to remove graph parameter(s) from
	 * @param ParamNames - An array of existing param names that will be removed.
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static bool RemoveGraphParams(UPCGGraph* Graph, const TArray<FName>& ParamNames);

	/**
	 * Returns the schema for a PCG Graph's graph parameters
	 *
	 * @param Graph - Graph to get the graph parameters of.
	 * @return FPCGGraphSchema with graph name, params schema (JSON), input/output pins
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static FPCGGraphSchema GetGraphSchema(const UPCGGraph* Graph);

	/**
	 * Returns the description of a PCG graph.
	 *
	 * @param Graph - Graph to get the description of.
	 * @return The graph's description text, or empty string if the graph has no description set.
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static FString GetGraphDescription(const UPCGGraph* Graph);

	/**
	 * Set the description of a PCGGraph
	 *
	 * @param Graph - Graph to set description of.
	 * @param Description - New description of graph
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphParams")
	static bool SetGraphDescription(UPCGGraph* Graph, const FString& Description);

	// Graph Instance --------------------------------------------------------------------------------------------------
	/**
	 * Gets all actors with a PCG graph instance in the scene.
	 *
	 * @return Array of FPCGGraphInstanceInfo for all actors with a graph instance
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphInstance")
	static TArray<FPCGGraphInstanceInfo> ListGraphInstances();

	/**
	* Spawns a PCG Volume with associated Graph Instance into the scene, optionally with Graph Param overrides.
	*
	* @param Graph - The PCG graph to spawn an instance of
	* @param Name - The name of the PCGVolume actor to spawn in the scene.
	* @param Transform - The transform to use for the new PCGVolume actor. Place at the origin unless there is a reason not to and use default scale3D of {"x": 25,"y": 25,"z": 10}
	* @param JsonParams - (Optional) JSON string representing JsonObject for the params to set. MUST be in format: {{"property_1_name": "property_1_value"}, ...}
	*	The default values for the graph params will be used if not set.
	* @return UStruct with information of the created actor and the corresponding graph instance
	*/

	UFUNCTION(meta = (AICallable), Category = "PCG|GraphInstance")
	static FPCGGraphInstanceInfo SpawnGraphInstance(UPCGGraph* Graph, const FString& Name, const FTransform& Transform, const FString& JsonParams = "");

	// Graph Instance Params --------------------------------------------------------------------------------------------
	/**
	 * Executes the graph instance and returns any issues encountered during execution.
	 *
	 * @param PCGVolume - The PCG Volume whose graph instance to execute.
	 * @return Array of messages emitted while executing the graph instance (empty on success with no issues)
	 */
	UFUNCTION(meta = (AICallable, NonTransactableToolCall), Category = "PCG|GraphInstanceParams")
	static UPCGExecuteGraphInstanceAsyncResult* ExecuteGraphInstance(const APCGVolume* PCGVolume); // @todo_pcg: Running on a PCG Volume isn't ideal long term

	/**
	 * Gets the graph instance params of a specific actor, actor MUST have a graph instance
	 *
	 * @param PCGVolume - The actor to get graph instance params from
	 * @return FInstancedPropertyBag containing the filtered graph instance parameters
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphInstanceParams")
	static FInstancedPropertyBag GetGraphInstanceParams(const APCGVolume* PCGVolume);

	/**
	 * Sets the graph instance params of a specific actor, actor MUST have a graph instance
	 *
	 * @param PCGVolume - The actor to set graph instance params for
	 * @param JsonParams - JSON string representing JsonObject for the params to set. MUST be in format: {{"property_1_name": "property_1_value"}, ...}
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphInstanceParams")
	static bool SetGraphInstanceParams(const APCGVolume* PCGVolume, const FString& JsonParams);

	/**
	 * Resets the given graph instance params back to the graph's default values. Actor MUST have a graph instance.
	 *
	 * @param PCGVolume - The actor to reset graph instance params for.
	 * @param ParamNames - Names of the parameters to reset.
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|GraphInstanceParams")
	static bool ResetGraphInstanceParams(const APCGVolume* PCGVolume, const TArray<FName>& ParamNames);

	// Nodes ----------------------------------------------------------------------------------------------------------
	/**
	 * Returns a list of available native PCG node type names.
	 *
	 * @param bCommonOnly Whether to only return the commonly used Native nodes. Only use bCommonOnly = false if the user specifically asks for it.
	 * @return Array of native node type names
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static TArray<FString> ListNativeNodes(const bool bCommonOnly = true);

	/**
	 * Lists the PCG graphs that can be used with the Subgraph native node. Only these graphs should be used with
	 * the Subgraph native node.
	 *
	 * @return Array of graph asset paths that can be used with the Subgraph native node
	*/
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static TArray<FString> ListAvailableSubgraphs();

	/**
	 * Returns the schema for a PCG node type including input/output pins,
	 * parameters, and their types.
	 *
	 * @param NodeName - Node name
	 * @return FPCGNativeNodeSchema with node name, description, properties schema (JSON), input/output pins
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static FPCGNativeNodeSchema GetNativeNodeSchema(const FString& NodeName);

	/**
	 * Adds a native node to the graph. 
	 *
	 * @param Graph - Graph to modify.
	 * @param NativeNodeType - The native type of the added node.
	 * @param NodeName - The name of the added node. (Must be unique identifier in the graph)
	 * @param JsonParams - The Json string representing a dictionary of the params to set on the node. Optional. Default is empty. Only non-default params need be included.
	 * @param NodeTitle - The Display Title of the node. Optional. Default is empty.
	 * @param NodeComment - The comment attached to the node, if needed. Default is empty.
	 * @param XPositionIdx - The X coordinate of the position of the node in the editor. Optional. Default is 0. Typical node size X is 200
	 * @param YPositionIdx - The Y coordinate of the position of the node in the editor. Optional. Default is 0. Typical node size Y is 100
	 * @return The Added Node object
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static UPCGNode* AddNode(
		UPCGGraph* Graph,
		const FString& NativeNodeType,
		const FString& NodeName,
		const FString& JsonParams = FString(),
		const FString& NodeTitle = FString(),
		const FString& NodeComment = FString(),
		const int32 XPositionIdx = 0,
		const int32 YPositionIdx = 0);

	/**
	 * Adds a subgraph node to the graph. 
	 *
	 * @param Graph - Graph to modify.
	 * @param SubGraphForNode - The subgraph to use in the added node.
	 * @param NodeName - The name of the added node. (Must be unique identifier in the graph)
	 * @param JsonParams - The Json string representing a dictionary of the params to override on the graph. Optional. Default is empty. Only non-default params need be included
	 * @param NodeTitle - The Display Title of the node. Optional. Default is empty.
	 * @param NodeComment - The comment attached to the node, if needed. Default is empty and will clear the comment.
	 * @param XPositionIdx - The X coordinate of the position of the node in the editor. Optional. Default is 0. Typical node size X is 200
	 * @param YPositionIdx - The Y coordinate of the position of the node in the editor. Optional. Default is 0. Typical node size Y is 100
	 * @return The Added Node object
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static UPCGNode* AddSubgraphNode(
		UPCGGraph* Graph,
		UPCGGraph* SubGraphForNode,
		const FString& NodeName,
		const FString& JsonParams = FString(),
		const FString& NodeTitle = FString(),
		const FString& NodeComment = FString(),
		const int32 XPositionIdx = 0, 
		const int32 YPositionIdx = 0);

	/**
	 * Updates a node by changing its params and/or title.
	 *
	 * @param Node - The node to modify.
	 * @param JsonParams - The Json string representing a dictionary of the params to override. Optional. Default is empty. Only non-default params need be included
	 * @param NodeTitle - The Display Title of the node. Optional. Default is empty. Empty will keep the old title
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static bool UpdateNode(UPCGNode* Node, const FString& JsonParams = FString(), const FString& NodeTitle = FString());

	/**
	 * Change the comment on the specified node.
	 *
	 * @param Node - The node to set the comment on. 
	 * @param NodeComment - The comment attached to the node.
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static bool SetNodeComment(UPCGNode* Node, const FString& NodeComment);
	
	/**
	 * Returns node details including name, position, and all parameter values.
	 *
	 * @param Node - The node to get info from.
	 * @return FPCGNodeInfo with node details including name, position, type, and parameter overrides
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static FPCGNodeInfo GetNodeInfo(const UPCGNode* Node);

	/**
	 * Change the position of node. 
	 *
	 * @param Node - The node to modify the position. 
	 * @param XPositionIdx - The X coordinate of the position of the node in the editor. 
	 * @param YPositionIdx - The Y coordinate of the position of the node in the editor.
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static bool RepositionNode(UPCGNode* Node, const int32 XPositionIdx, const int32 YPositionIdx);

	/**
	 * Removes the node from the graph, will also remove edges connected to the node. 
	 *
	 * @param Graph - The graph to remove the node from. 
	 * @param Node - The node to remove. 
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static bool RemoveNode(UPCGGraph* Graph, UPCGNode* Node);

	/**
	 * Add an edge between two nodes connected to the specified pins. 
	 *
	 * @param FromNode - The source node of the edge to add. 
	 * @param FromPinLabel - The label of the source pin of the source node. 
	 * @param ToNode - The destination node of the edge to add. 
	 * @param ToPinLabel - The label of the destination pin of the destination node. 
	 * @return The list of nodes (conversions and/or filters) that were added to allow the connection to work. 
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static TArray<UPCGNode*> ConnectNodePins(UPCGNode* FromNode, const FString& FromPinLabel, UPCGNode* ToNode, const FString& ToPinLabel);


	/**
	 * Removes the edge between two nodes connected to the specified pins. 
	 *
	 * @param FromNode - The source node of the edge to add. 
	 * @param FromPinLabel - The label of the source pin of the source node. 
	 * @param ToNode - The destination node of the edge to add. 
	 * @param ToPinLabel - The label of the destination pin of the destination node.
	 * @return boolean representing success/failed 
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Nodes")
	static bool DisconnectNodePins(UPCGNode* FromNode, const FString& FromPinLabel, UPCGNode* ToNode, const FString& ToPinLabel);

	/**
	 * Returns a JSON Data View of a specific node's output data from the last graph execution.
	 * On first call, enables inspection so future ExecuteGraphInstance calls store per-node data.
	 * If no inspection data exists, returns an error prompting re-execution.
	 *
	 * IMPORTANT: Inspection state is shared at the graph asset level. If multiple actors use the
	 * same graph, you MUST call this tool (and ExecuteGraphInstance) on only one actor at a time.
	 * Wait for each call to fully complete before calling on the next actor. Concurrent calls on
	 * actors sharing the same graph will cause a freeze.
	 *
	 * @param PCGVolume - The PCG Volume whose graph was executed.
	 * @param Node - The node whose output to inspect.
	 * @param PinLabel - Output pin label to read. Defaults to "Out".
	 * @param AttributeName - Filter to a single attribute/property (e.g. "$Position", "$Density", "MyCustomAttr"). Empty = all attributes.
	 * @param StartIndex - Element range start, inclusive, 0-based. Default 0.
	 * @param EndIndex - Element range end, exclusive. -1 means all elements (Python slice convention). Default -1.
	 * @return JSON string with the data view contents
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Data")
	static FString GetNodeDataView(
		const APCGVolume* PCGVolume,
		const UPCGNode* Node,
		const FString& PinLabel = TEXT("Out"),
		const FString& AttributeName = FString(),
		const int32 StartIndex = 0,
		const int32 EndIndex = -1);

	/* --------------------------------------------------------------------------------------------------------------- */

	/**
	 * Adds a comment box around the given nodes. 
	 *  Note: If the bounding box of the nodes include other nodes, they will be included in the comment.
	 *
	 * @param Graph - Graph to add comment to.
	 * @param Nodes - The list of nodes to include in the comment box.
	 * @param Comment - The comment to put on the comment box.
	 * @param Color - The color of the comment box. Defaults to White
	 * @return The unique id of the comment node. (Can be used to edit/remove the comment)
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|CommentBox")
	static FString AddCommentBox(UPCGGraph* Graph, const TArray<UPCGNode*>& Nodes, const FString& Comment, FLinearColor Color = FLinearColor::White);

	/**
	 * Updates an existing comment box with new nodes and value. 
	 *  Note: If the bounding box of the nodes include other nodes, they will be included in the comment.
	 *
	 * @param Graph - Graph to add comment to.
	 * @param CommentId - The unique id of the comment to update.
	 * @param Nodes - The list of nodes to include in the comment box. If empty, the box will keep its current dimensions.
	 * @param Comment - The new comment to put on the comment box. If empty, will keep the same text as it was. Default is empty.
	 * @param Color - The color of the comment box. Defaults to White
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|CommentBox")
	static bool UpdateCommentBox(
		UPCGGraph* Graph,
		const FString& CommentId,
		const TArray<UPCGNode*>& Nodes,
		const FString& Comment = FString(),
		FLinearColor Color = FLinearColor::White);

	/**
	 * Removes a comment box from the graph. Does not affect the nodes it contains.
	 *
	 * @param Graph - Graph to remove the comment from.
	 * @param CommentId - The unique id of the comment to remove.
	 * @return boolean representing success/failed
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|CommentBox")
	static bool RemoveCommentBox(UPCGGraph* Graph, const FString& CommentId);

	/**
	 * Triggers the user to draw a spline in the viewport to be used later in the world building. Waits for the user to be done. 
	 *
	 * @param ActorLabel - The label of the actor created or if bRedraw, the label of the actor to redraw the spline
	 * @param ActorTag - Tag assigned to the actor
	 * @param bRedraw - false: creates a new actor with the spline. true: find an actor with the label, and replaces its spline.
	 * @param bClosedSpline - true: closed spline (region) false: path.
	 */
	UFUNCTION(meta = (AICallable), Category = "PCG|Other")
	static UToolCallAsyncResultVoid* DrawSpline(const FString& ActorLabel, const FString& ActorTag, bool bRedraw, bool bClosedSpline);
};
