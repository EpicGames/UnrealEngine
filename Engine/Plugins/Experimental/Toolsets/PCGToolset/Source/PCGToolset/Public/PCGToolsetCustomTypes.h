// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "PCGVolume.h"
#include "PCGGraph.h"
#include "ToolsetRegistry/ToolCallAsyncResult.h"
#include "PCGToolsetCustomTypes.generated.h"

USTRUCT()
struct FPCGGraphInstanceInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<APCGVolume> Actor = nullptr;

	UPROPERTY()
	TObjectPtr<UPCGGraph> Graph = nullptr;
};

USTRUCT()
struct FPCGParamDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	EPCGMetadataTypes Type = EPCGMetadataTypes::Unknown;

	UPROPERTY()
	FString Description;

	UPROPERTY()
	EPropertyBagContainerType ContainerType = EPropertyBagContainerType::None;

	UPROPERTY()
	FString DefaultValueJson = TEXT("");
};

USTRUCT()
struct FPCGPinInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FString Description;
};

USTRUCT()
struct FPCGEdgeInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString SrcNode;

	UPROPERTY()
	FString SrcPin;

	UPROPERTY()
	FString DestNode;

	UPROPERTY()
	FString DestPin;
};

USTRUCT()
struct FPCGNodeInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Path;

	UPROPERTY()
	FString Position;

	UPROPERTY()
	FString Title;

	UPROPERTY()
	bool bEnabled = true;

	UPROPERTY()
	FString Comment;

	UPROPERTY()
	FString NodeType;

	UPROPERTY()
	FInstancedPropertyBag ParamOverrides;
};

USTRUCT()
struct FPCGGraphStructure
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Description;

	UPROPERTY()
	TArray<FPCGNodeInfo> Nodes;

	UPROPERTY()
	TArray<FPCGEdgeInfo> Edges;
};

USTRUCT()
struct FPCGGraphSchema
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString GraphParamsSchema;

	UPROPERTY()
	TArray<FPCGPinInfo> InputPins;

	UPROPERTY()
	TArray<FPCGPinInfo> OutputPins;
};

USTRUCT()
struct FPCGNativeNodeSchema
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Description;

	UPROPERTY()
	FString PropertiesSchema;

	UPROPERTY()
	TArray<FPCGPinInfo> InputPins;

	UPROPERTY()
	TArray<FPCGPinInfo> OutputPins;
};

USTRUCT()
struct FPCGNodeExecutionMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FString Message;

	UPROPERTY()
	FString Severity;

	UPROPERTY()
	FString ReporterNode;
};

UCLASS()
class UPCGExecuteGraphInstanceAsyncResult : public UToolCallAsyncResult
{
	GENERATED_BODY()

public:
	bool SetValue(TArray<FPCGNodeExecutionMessage>&& InValue)
	{
		return MaybeBroadcastSuccessfulCompletion(MoveTemp(InValue), Value);
	}

	UPROPERTY()
	TArray<FPCGNodeExecutionMessage> Value;
};