// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGGraph.h"
#include "PCGVolume.h"
#include "PCGToolsetCustomTypes.h"
#include "PCGSubgraph.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

namespace PCGToolsetLibrary
{
	namespace Json
	{
		TSharedPtr<FJsonObject> ParseJson(const FString& JsonString);
		FString ToJsonString(const TSharedPtr<FJsonObject>& JsonObject);
		FString ToJsonString(const TArray<TSharedPtr<FJsonValue>>& JsonArray);
	}

	namespace Graph
	{
		// Clones SourceBag, excluding internal PCG params (see Constants::PrimitiveInternalParamNames).
		FInstancedPropertyBag BuildFilteredBag(const FInstancedPropertyBag& SourceBag);

		// Marks each top-level key in JsonKeys as an override on the graph instance's parameter bag.
		void EnablePropertyOverrides(UPCGGraphInstance* GraphInstance, const TSharedPtr<FJsonObject>& JsonKeys);

		// True for PCG-internal params that should never be shown or edited by the LLM.
		bool IsPrimitiveInternalParam(const FProperty* Property);

		// Re-raises captured errors/warnings from a scoped call as script errors to the tool caller.
		void RaiseScopedErrors(const PCGUtils::FScopedCall& ScopedCall);

		// Maps a log verbosity to the short severity label used in FPCGNodeExecutionMessage.
		FString VerbosityToString(ELogVerbosity::Type Verbosity);

		// Returns Graph's user-parameters bag, minus internal params.
		FInstancedPropertyBag GetGraphParams(const UPCGGraph* Graph);

		// Returns only the params the subgraph node overrides from its graph defaults.
		FInstancedPropertyBag GetSubgraphNodeParamOverrides(const UPCGSubgraphSettings* SubgraphSettings, const UPCGGraph* Subgraph);

		// Applies JSON param values to the graph instance's overrides bag and marks each as overridden.
		bool SetGraphInstanceParams(UPCGGraphInstance* GraphInstance, const FString& JsonParams);

		// Returns FPCGNodeInfo for every node in Graph (including the synthetic input/output nodes).
		TArray<FPCGNodeInfo> GetGraphNodesInfo(const UPCGGraph* Graph);

		// Returns the editable, LLM-visible FPropertys defined on InSettingsClass (stops at UPCGSettings).
		TArray<FProperty*> GetNodePropertiesFromSettings(TSubclassOf<UPCGSettings> InSettingsClass);

		// Returns the cached "display title -> default settings" lookup for every exposed PCG settings subclass.
		const TMap<FName, UPCGSettings*>& GetNodeNameToSettingsMap();

		// Returns the display-ready info (name, position, type, overrides) for a single node.
		FPCGNodeInfo GetNodeInfo(const UPCGNode* Node);

		// Converts PCG pin properties into FPCGPinInfo records for schema output.
		TArray<FPCGPinInfo> GetNodePinsSchema(const TArray<FPCGPinProperties>& PinProperties);

		// Returns every edge in Graph as FPCGEdgeInfo (src/dest node + pin labels).
		TArray<FPCGEdgeInfo> GetGraphEdges(const UPCGGraph* Graph);

		// Returns a cached fresh instance of PCGSettingClass (avoids stale CDO defaults).
		UPCGSettings* GetRealDefaultObject(UClass* PCGSettingClass);

		// Finds PCG graph assets in the given package paths, optionally filtered by a path-substring predicate.
		TArray<FString> FindGraphPaths(const TSet<FName>& PackagePaths, TFunctionRef<bool(const FString& /*PathName*/)> PathPredicate = [](const FString&) { return true; });
	}

	namespace Constants
	{
		static const TSet<FString> PrimitiveInternalParamNames = {
			TEXT("InputsProcessingGraph"), TEXT("InputsFallbackGraph"), TEXT("CoreProcessGraph"), TEXT("PrepareOutputsGraph")
		};

		static const TSet<FString> CommonNativeNodes = {
			TEXT("Add Attribute"),TEXT("Add Component"),TEXT("Attribute Bitwise Op"),TEXT("Attribute Boolean Op"),
			TEXT("Attribute Cast"),TEXT("Attribute Compare Op"),TEXT("Attribute Maths Op"),TEXT("Attribute Noise"),
			TEXT("Attribute Partition"),TEXT("Attribute Reduce"),TEXT("Attribute Remap"),TEXT("Attribute Remove Duplicates"),
			TEXT("Attribute Rename"),TEXT("Attribute Rotator Op"),TEXT("Attribute Select"),TEXT("Attribute String Op"),
			TEXT("Attribute Transform Op"),TEXT("Attribute Trig Op"),TEXT("Attribute Vector Op"),TEXT("Bounds Modifier"),
			TEXT("Branch"),TEXT("Combine Points"),TEXT("Copy Attributes"),TEXT("Copy Points"),TEXT("Create Constant"),
			TEXT("Create Points"),TEXT("Create Points Grid"),TEXT("Create Points Sphere"),TEXT("Create Polygon 2D"),
			TEXT("Create Spline"),TEXT("Create Surface From Polygon2D"),TEXT("Create Surface From Spline"),
			TEXT("Delete Attributes"),TEXT("Difference"),TEXT("Distance"),TEXT("Duplicate Point"),TEXT("Extents Modifier"),
			TEXT("Extract Attribute"),TEXT("Filter Attribute Elements"),TEXT("Filter Attribute Elements by Range"),
			TEXT("Filter Data - Any"),TEXT("Filter Data By Attribute"),TEXT("Filter Data By Index"),
			TEXT("Filter Elements By Index"),TEXT("Get Actor Data"),TEXT("Get Actor Property"),
			TEXT("Get Attribute From Point Index"),TEXT("Get Attribute Set from Index"),TEXT("Get Landscape Data"),
			TEXT("Get PCG Component Data"),TEXT("Get Property From Object Path"),TEXT("Get Segment"),
			TEXT("Get Spline Control Points"),TEXT("Get Spline Data"),TEXT("Get Texture Data"),
			TEXT("Get Volume Data"),TEXT("Intersection"),TEXT("Load PCG Data Asset"),TEXT("Loop"),
			TEXT("Make Concrete"),TEXT("Make Rotator Attribute"),TEXT("Make Transform Attribute"),
			TEXT("Make Vector Attribute"),TEXT("Match And Set Attributes"),TEXT("Merge Points"),
			TEXT("Offset Polygon"),TEXT("Offset Spline"),TEXT("Pathfinding"),TEXT("Point From Mesh"),
			TEXT("Point Neighborhood"),TEXT("Point To Attribute Set"),TEXT("Polygon Operation"),
			TEXT("Projection"),TEXT("Random Choice"),TEXT("Sample Texture"),TEXT("Save Data View"),
			TEXT("Select"),TEXT("Select Grammar"),TEXT("Self Pruning"),TEXT("Sort Attributes"),TEXT("Spatial Noise"),
			TEXT("Spawn Actor"),TEXT("Spawn Spline Component"),TEXT("Spawn Spline Mesh"),TEXT("Spline Direction"),
			TEXT("Spline Intersection"),TEXT("Spline Sampler"),TEXT("Spline to Segment"),TEXT("Split Points"),
			TEXT("Static Mesh Spawner"),TEXT("Subdivide Segment"),TEXT("Subdivide Spline"),TEXT("Subgraph"),
			TEXT("Surface Sampler"),TEXT("Switch"),TEXT("To Data View"),TEXT("To Point"),TEXT("Transform Points"),
			TEXT("Union"),TEXT("Volume Sampler"),TEXT("World Ray Hit Query"),TEXT("World Raycast"),TEXT("World Volumetric Query")
		};

		// Helpers to get directory settings
		TSet<FName> GetSubgraphDirectories();
		TSet<FName> GetExamplesDirectories();
		TSet<FName> GetInstantGraphDirectories();
	}

}
