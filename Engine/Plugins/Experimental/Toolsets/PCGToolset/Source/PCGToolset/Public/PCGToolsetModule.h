// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

namespace UE::PCGToolset
{
	class FPCGAttributePropertySelectorConverter;
}

DECLARE_LOG_CATEGORY_EXTERN(LogPCGToolset, Log, All);

class FPCGToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<UE::PCGToolset::FPCGAttributePropertySelectorConverter> SelectorConverter;
};
