// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "PCGToolsetSettings.generated.h"

/**
 * Settings for the PCGToolset plugin
 */
UCLASS(Config=Editor, DefaultConfig, meta=(DisplayName="PCG Toolset"))
class PCGTOOLSET_API UPCGToolsetSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPCGToolsetSettings();

	/**
	 * Directories to search for subgraph primitives that will be exposed to the AI.
	 * These graphs should be reusable building blocks for PCG graph creation.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Primitives", meta = (ContentDir, LongPackageName, DisplayName = "Subgraph Directories", Tooltip = "Directories containing PCG subgraph primitives to expose to the AI"))
	TArray<FDirectoryPath> SubgraphDirectories;

	/**
	 * Directories to search for example PCG graphs that will be shown to the AI.
	 * These graphs serve as learning examples for the AI to understand patterns.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Examples", meta = (ContentDir, LongPackageName, DisplayName = "Example Graph Directories", Tooltip = "Directories containing example PCG graphs for AI reference"))
	TArray<FDirectoryPath> ExampleGraphDirectories;

	/**
	 * Directories to search for instant PCG graphs that will be exposed to the AI.
	 * These graphs are ready-to-execute templates the AI can run directly.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Instants", meta = (ContentDir, LongPackageName, DisplayName = "Instant Graph Directories", Tooltip = "Directories containing instant PCG graphs for AI execution"))
	TArray<FDirectoryPath> InstantGraphDirectories;

	// UDeveloperSettings interface
	virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FText GetSectionDescription() const override;
#endif
};
