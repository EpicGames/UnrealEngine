// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PCGToolset.h"
#include "PCGToolsetLibraryCore.h"
#include "ToolsetRegistry/AgentSkill.h"

#include "PCGToolsetSkills.generated.h"

UCLASS()
class UPCGGraphGenerationSkill : public UAgentSkill
{
	GENERATED_BODY()

protected:
	virtual FString GeneratePrompt_Implementation(const FString& InitialInstructions) const override
	{
		FString CustomInstructions = InitialInstructions;
		CustomInstructions.ReplaceInline(TEXT("{Subgraphs}"),
			*FString::Join(UPCGToolset::ListAvailableSubgraphs(), TEXT(", ")));
		CustomInstructions.ReplaceInline(TEXT("{Nodes}"),
			*FString::Join(UPCGToolset::ListNativeNodes(true), TEXT(", ")));
		CustomInstructions.ReplaceInline(TEXT("{Examples}"),
			*FString::Join(PCGToolsetLibrary::Graph::FindGraphPaths(
				PCGToolsetLibrary::Constants::GetExamplesDirectories()), TEXT(", ")));
		CustomInstructions.ReplaceInline(TEXT("{Instant Graphs}"),
			*FString::Join(PCGToolsetLibrary::Graph::FindGraphPaths(
				PCGToolsetLibrary::Constants::GetInstantGraphDirectories()), TEXT(", ")));
		return CustomInstructions;
	}
};
